// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
    public class AxFImporter : ModuleRules
    {
        private string ThirdPartyPath
        {
            get { return Path.GetFullPath(Path.Combine(EngineDirectory, "Restricted/NotForLicensees/Source/ThirdParty/Enterprise")); }
        }

        public AxFImporter(ReadOnlyTargetRules Target) : base(Target)
        {
			bLegalToDistributeObjectCode = true;

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Analytics",
                    "Core",
                    "RenderCore",
                    "ImageCore",
                    "CoreUObject",
                    "Engine",
                    "MessageLog",
					"EditorFramework",
                    "UnrealEd",
                    "Slate",
                    "SlateCore",
                    "MainFrame",
                    "InputCore",
                    "MaterialEditor",
                    "Projects"
                }
            );

            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                }
            );

			string AfxSdkVersion = "1.8.1";

            if (Target.Platform == UnrealTargetPlatform.Win64)
            {
				string AxFDllName = $"AxFDecoding.{AfxSdkVersion}.dll";

				PublicDelayLoadDLLs.Add(AxFDllName);
				string ModulePath = Path.Combine(EngineDirectory, "Plugins/Enterprise/AxFImporter/Binaries/ThirdParty/AxF", Target.Platform.ToString(), AxFDllName);
				if (!File.Exists(ModulePath))
                {
					string Err = string.Format("AxF Decoding dll '{0}' not found.", ModulePath);
					System.Console.WriteLine(Err);
					throw new BuildException(Err);
                }
                RuntimeDependencies.Add(ModulePath);
            }

            if (Directory.Exists(ThirdPartyPath))
            {
                //third party libraries

                string[] Libs = { $"AxF-Decoding-SDK-{AfxSdkVersion}" };
                string[] StaticLibNames = { "AxFDecoding" };

                foreach (string Lib in Libs)
                {
                    string IncludePath = Path.Combine(ThirdPartyPath, Lib, "include");
                    if (Directory.Exists(IncludePath))
                    {
                        PublicSystemIncludePaths.Add(IncludePath);
                    }
                    else
                    {
                        return;
                    }
                }

				PrivateDefinitions.Add("USE_AXFSDK");
				PrivateDefinitions.Add($"AFX_SDK_VERSION=\"{AfxSdkVersion}\"");

				string TargetPlatform = "Windows.x64";
                string TargetExtension = "lib";
                if (Target.Platform == UnrealTargetPlatform.Win64)
                {
                    PublicDefinitions.Add("MI_PLATFORM_WINDOWS");
                }
                else if (Target.Platform == UnrealTargetPlatform.Linux)
                {
                    TargetPlatform = "Linux.x64";
                    TargetExtension = "so";
                }
                else if (Target.Platform == UnrealTargetPlatform.Mac)
                {
                    TargetPlatform = "MacOSX.x64";
                    TargetExtension = "dylib";
                }

                // add static libraries
                for (int i = 0; i < Libs.Length; ++i)
                {
                    string LibName = StaticLibNames[i];
                    if (LibName == "")
                    {
                        continue;
                    }

                    LibName += "." + TargetExtension;

                    string LibPath = Path.Combine(ThirdPartyPath, Libs[i], TargetPlatform, "lib", LibName);
                    PublicAdditionalLibraries.Add(LibPath);
                }
            }
        }
    }
}
