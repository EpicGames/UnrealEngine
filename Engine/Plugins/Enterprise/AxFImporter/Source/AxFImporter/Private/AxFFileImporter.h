// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/Logging.h"

#include "Containers/Map.h"
#include "UObject/ObjectMacros.h"

class UMaterialInterface;
class UAxFImporterOptions;
class UObject;
class UTextureFactory;
class UMaterialInterface;

class IAxFFileImporter
{
public:
	using FProgressFunc = TFunction<void(const FString& MsgName, int MaterialIndex)>;

	virtual ~IAxFFileImporter() {}

	virtual bool OpenFile(const FString& InFileName, const UAxFImporterOptions& InImporterOptions) = 0;
	virtual int GetMaterialCountInFile() = 0;
	virtual bool ImportMaterials(UObject* ParentPackage, EObjectFlags Flags, FProgressFunc ProgressFunc = nullptr) = 0;
	virtual void SetTextureFactory(UTextureFactory* TextureFactory) = 0;
	virtual TMap<FString, UMaterialInterface*> GetCreatedMaterials() = 0;
	virtual bool Reimport(const FString& InFileName, const UAxFImporterOptions& InImporterOptions, UMaterialInterface* OutMaterial) = 0;

	virtual const TArray<AxFImporterLogging::FLogMessage> GetLogMessages() = 0;
};
