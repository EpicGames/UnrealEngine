// Copyright Epic Games, Inc. All Rights Reserved.

#include "AxFImporter.h"

#ifdef USE_AXFSDK

// To avoid annoying VS' visual cue that AFX_SDK_VERSION is not defined
#ifndef AFX_SDK_VERSION
#define AFX_SDK_VERSION
#endif

#include "PackageTools.h"
#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/Texture2D.h"
#include "Misc/Paths.h"

#include "MaterialEditingLibrary.h"

#include "Materials/Material.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionSquareRoot.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionArccosine.h"
#include "Materials/MaterialExpressionClamp.h"
#include "Materials/MaterialExpressionDotProduct.h"
#include "Materials/MaterialExpressionNormalize.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionFresnel.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionPixelNormalWS.h"
#include "Materials/MaterialExpressionVertexNormalWS.h"
#include "Materials/MaterialExpressionCameraVectorWS.h"
#include "Materials/MaterialExpressionIf.h"
#include "Materials/MaterialExpressionStaticSwitch.h"
#include "Materials/MaterialExpressionFloor.h"
#include "Materials/MaterialExpressionFrac.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionScalarParameter.h"

#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunction.h"


#include "AxF/decoding/AxF_basic_io.h"
#include "AxF/decoding/TextureDecoder.h"

#include "common/Logging.h"
#include "Factories/TextureFactory.h"
#include "AxFImporterOptions.h"

#include "ImageCore.h"
#include "Async/ParallelFor.h"

class FAxFLog {
public:
	void Error(const FString& Message)
	{
		UE_LOGF(LogAxFImporter, Error, "%ls", *Message);
		Messages.Emplace(AxFImporterLogging::EMessageSeverity::Error, Message);
	}

	void Warn(const FString& Message)
	{
		UE_LOGF(LogAxFImporter, Warning, "%ls", *Message);
		Messages.Emplace(AxFImporterLogging::EMessageSeverity::Warning, Message);
	}

	void Info(const FString& Message)
	{
		UE_LOGF(LogAxFImporter, Display, "%ls", *Message);
	}

	TArray<AxFImporterLogging::FLogMessage> Messages;
};

class FAxFFileImporter : public IAxFFileImporter 
{
public:

	class FAxFRepresentation
	{
	public:
		axf::decoding::AXF_REPRESENTATION_HANDLE Handle;
		FString ClassName;

	};

	struct FAxFTextureSource
	{
		// @todo Oodle : just use float here, convert later ; replace this whole class with FImage
		TArray<FFloat16> Pixels;

		int32 Width;
		int32 Height;
		int32 Depth;
		int32 ChannelCount;// Source channel count, not what's stored in Pixels(it's 4 always)

		void Init(int32 InWidth, int32 InHeight, int32 InDepth, int32 InChannelCount)
		{
			check(1 == InDepth);
			check(4 >= InChannelCount);
			Width = InWidth;
			Height = InHeight;
			Depth = InDepth;
			ChannelCount = InChannelCount;
			Pixels.SetNumUninitialized(Width * Height * InDepth * 4);
		}

		int32 GetChannelCount()
		{
			return ChannelCount;
		}

		int32 GetPixelCount()
		{
			return Width * Height * Depth;
		}

		void SetPixel(int32 PixelIndex, FLinearColor C)
		{
			Pixels[PixelIndex * 4 + 0].Set(C.R);
			Pixels[PixelIndex * 4 + 1].Set(C.G);
			Pixels[PixelIndex * 4 + 2].Set(C.B);
			Pixels[PixelIndex * 4 + 3].Set(C.A);
		}

		FLinearColor GetPixel(int32 PixelIndex) const
		{
			FLinearColor C;
			C.R = Pixels[PixelIndex * 4 + 0].GetFloat();
			C.G = Pixels[PixelIndex * 4 + 1].GetFloat();
			C.B = Pixels[PixelIndex * 4 + 2].GetFloat();
			C.A = Pixels[PixelIndex * 4 + 3].GetFloat();
			return C;
		}

		FVector GetRGB(int32 PixelIndex)
		{
			FLinearColor C = GetPixel(PixelIndex);
			return FVector(C.R, C.G, C.B);
		}

		void SetRGB(int32 PixelIndex, FVector C)
		{
			Pixels[PixelIndex * 4 + 0].Set(C.X);
			Pixels[PixelIndex * 4 + 1].Set(C.Y);
			Pixels[PixelIndex * 4 + 2].Set(C.Z);
		}


		const uint8* GetRawPixelData()
		{
			return (const uint8*)Pixels.GetData();
		}

	};


	struct FFlakesTextureAtlas
	{
		int32 TextureWidth = 1;
		int32 TextureHeight = 1;

		int32 SliceWidth = 1;
		int32 SliceHeight = 1;

		int32 FlakesTextureSlicesPerRow = 1;
	};

	struct FProcessedTextureSource
	{
		UTexture2D* Texture = nullptr;
		FVector2D Tiling = FVector2D(1.f, 1.f);
		FVector Scale = FVector(1.f, 1.f, 1.f);

		// Multiplying colors by value bigger than this will have them clamped(e.g. diffuse is clamped by one in the shader)
		float MaxColorFactorToAvoidClamping = FLT_MAX;

		EMaterialSamplerType SamplerType = SAMPLERTYPE_LinearColor;
		FVector Constant = FVector(1.f, 1.f, 1.f);
	};

	class ICreatedMaterial
	{
	public:

		virtual ~ICreatedMaterial() {}


		virtual void Build() = 0;

		virtual void SetTextureDiffuseColor(FProcessedTextureSource) = 0;

		virtual void SetTextureAlpha(FProcessedTextureSource) = 0;

		virtual void SetTextureNormal(FProcessedTextureSource) = 0;

		virtual void SetTextureSpecularColor(FProcessedTextureSource) = 0;
		virtual void SetTextureFresnel(FProcessedTextureSource) = 0;
		virtual void SetTextureSpecularLobe(FProcessedTextureSource) = 0;
		virtual void SetIsAnisotropic(bool InIsAnisotropic) = 0;
		virtual void SetHasFresnel(bool InHasFresnel) = 0;

		virtual void SetTextureClearcoatColor(FProcessedTextureSource) = 0;
		virtual void SetTextureClearcoatIOR(FProcessedTextureSource) = 0;
		virtual void SetTextureClearcoatNormal(FProcessedTextureSource) = 0;

		virtual void SetTextureCarpaint2BRDFColors(FProcessedTextureSource) = 0;
		virtual void SetTextureCarpaint2BTFFlakes(FProcessedTextureSource, FFlakesTextureAtlas) = 0;
		virtual void SetTextureCarpaintClearcoatNormal(FProcessedTextureSource) = 0;

		virtual void SetTextureHeight(FProcessedTextureSource ProcessedTextureSource) = 0;

		void SetName(FString InName)
		{
			Name = InName;
		}

		FString Name;

		void SetRepresentationHandle(axf::decoding::AXF_REPRESENTATION_HANDLE RepresentationHandle)
		{
			Representation.Handle = RepresentationHandle;
		}

		void SetRepresentationClassName(FString RepresentationClassName)
		{
			Representation.ClassName = RepresentationClassName;
		}

		axf::decoding::AXF_REPRESENTATION_HANDLE GetRepresentationHandle()
		{
			return Representation.Handle;
		}

		FAxFRepresentation Representation;

		void SetMaterialIntefrace(UMaterial* NewMaterial)
		{
			Material = NewMaterial;
		}

		UMaterial* GetMaterialInterface()
		{
			return Material;
		}

		UMaterial* Material = nullptr;


		void SetNoRefraction(bool PropertyValue)
		{
			bNoRefraction = PropertyValue;
		}

		void SetCT_F0s(TArray<float> PropertyValue)
		{
			CT_F0s = PropertyValue;
		}

		void SetCT_Coeffs(TArray<float> PropertyValue)
		{
			CT_Coeffs = PropertyValue;
		}

		void SetCT_Spreads(TArray<float> PropertyValue)
		{
			CT_Spreads = PropertyValue;
		}

		void SetCT_Diffuse(float PropertyValue)
		{
			CT_Diffuse = PropertyValue;
		}

		void SetCC_IOR(float PropertyValue)
		{
			CC_IOR = PropertyValue;
		}

		void SetFlakesMaxThetaI(int PropertyValue)
		{
			FlakesMaxThetaI = PropertyValue;
		}

		void SetFlakesNumThetaF(int PropertyValue)
		{
			FlakesNumThetaF = PropertyValue;
		}

		void SetFlakesNumThetaI(int PropertyValue)
		{
			FlakesNumThetaI = PropertyValue;
		}

		void SetFlakesThetaFISliceLUT(UTexture2D* Texture)
		{
			FlakesThetaFISliceLUT = Texture;
		}

	protected:


		UMaterialFunction* CreateFunction(const FString& FunctionName, const TFunction<void(UMaterialFunction*)>&  Generate)
		{
			const FString& AssetPath = TEXT("/Game/AxF");

			FString   FunctionPackageName = UPackageTools::SanitizePackageName(*(AssetPath / FunctionName));
			UPackage* Package = CreatePackage(*FunctionPackageName);

			UMaterialFunction* Function = NewObject<UMaterialFunction>(Package, UMaterialFunction::StaticClass(), FName(*Name), RF_Public | RF_Standalone);

			check(Function);
			Function->StateId = FGuid::NewGuid();

			Generate(Function);

			// Arrange editor nodes
			UMaterialEditingLibrary::LayoutMaterialFunctionExpressions(Function);

			Function->PostLoad();
			FAssetRegistryModule::AssetCreated(Function);
			Function->MarkPackageDirty();
			return Function;
		}

		template<class MaterialExpressionType>
		MaterialExpressionType* CreateMaterialExpression()
		{
			MaterialExpressionType* Expression = NewObject<MaterialExpressionType>(Material);

			if (Material->IsA<UMaterial>())
			{
				Material->GetExpressionCollection().AddExpression(Expression);
				Expression->Material = Material;
			}
			// MaterialFunction->FunctionExpressions.Add(NewExpression);

			Expression->UpdateMaterialExpressionGuid(true, true);

			// use for params
			Expression->UpdateParameterGuid(true, true);
			if (Expression->HasAParameterName())
			{
				Expression->ValidateParameterName(false);
			}
			if (Material->IsA<UMaterial>())
			{
				Material->AddExpressionParameter(Expression, Material->EditorParameters);
			}

			Expression->MarkPackageDirty();

			return Expression;
		}

		class FMaterialExpressionWrapper
		{
		public:
			FMaterialExpressionWrapper(){}
			FMaterialExpressionWrapper(UMaterialExpression* Expression) :Expression(Expression), OutputIndex(0) {}
			FMaterialExpressionWrapper(UMaterialExpression* Expression, int32 OutputIndex) :Expression(Expression), OutputIndex(OutputIndex) {}

			UMaterialExpression* Expression = nullptr;
			int32 OutputIndex = 0;

			operator bool()
			{
				return Expression != nullptr;
			}

		};

		FMaterialExpressionWrapper Vec3(float R, float G, float B)
		{
			UMaterialExpressionConstant3Vector* Result = CreateMaterialExpression<UMaterialExpressionConstant3Vector>();
			Result->Constant.R = R;
			Result->Constant.G = G;
			Result->Constant.B = B;
			return Box(Result);
		}

		FMaterialExpressionWrapper If(FMaterialExpressionWrapper A, FMaterialExpressionWrapper B, FMaterialExpressionWrapper Less, FMaterialExpressionWrapper NotLess)
		{
			UMaterialExpressionIf* Result = CreateMaterialExpression<UMaterialExpressionIf>();
			Connect(Result->A, A);
			Connect(Result->B, B);
			Connect(Result->AGreaterThanB, NotLess);
			Connect(Result->ALessThanB, Less);
			return Result;
		}

		FMaterialExpressionWrapper Lerp(FMaterialExpressionWrapper A, FMaterialExpressionWrapper B, FMaterialExpressionWrapper Alpha)
		{
			UMaterialExpressionLinearInterpolate* Result = CreateMaterialExpression<UMaterialExpressionLinearInterpolate>();
			Connect(Result->A, A);
			Connect(Result->B, B);
			Connect(Result->Alpha, Alpha);
			return Result;
		}

		FMaterialExpressionWrapper Add(FMaterialExpressionWrapper A, FMaterialExpressionWrapper B)
		{
			UMaterialExpressionAdd* Result = CreateMaterialExpression<UMaterialExpressionAdd>();

			Connect(Result->A, A);
			Connect(Result->B, B);

			return Box(Result);
		}

		FMaterialExpressionWrapper Add(FMaterialExpressionWrapper A, float B)
		{
			UMaterialExpressionAdd* Result = CreateMaterialExpression<UMaterialExpressionAdd>();

			Connect(Result->A, A);
			Result->ConstB = B;

			return Box(Result);
		}

		UMaterialExpression* Add(UMaterialExpression* A, UMaterialExpression* B)
		{
			return UnBox(Add(Box(A), Box(B)));
		}

		FMaterialExpressionWrapper Sub(FMaterialExpressionWrapper A, FMaterialExpressionWrapper B)
		{
			UMaterialExpressionSubtract* Result = CreateMaterialExpression<UMaterialExpressionSubtract>();

			Connect(Result->A, A);
			Connect(Result->B, B);

			return Box(Result);
		}

		FMaterialExpressionWrapper Sub(float A, FMaterialExpressionWrapper B)
		{
			UMaterialExpressionSubtract* Product = CreateMaterialExpression<UMaterialExpressionSubtract>();
			Product->ConstA = A;
			Product->B.Connect(B.OutputIndex, B.Expression);
			return Box(Product);
		}

		FMaterialExpressionWrapper Dot(FMaterialExpressionWrapper A, FMaterialExpressionWrapper B)
		{
			UMaterialExpressionDotProduct* Result = CreateMaterialExpression<UMaterialExpressionDotProduct>();
			Connect(Result->A, A);
			Connect(Result->B, B);
			return Box(Result);
		}

		FMaterialExpressionWrapper Clamp(FMaterialExpressionWrapper V, float Min, float Max)
		{
			UMaterialExpressionClamp* Result = CreateMaterialExpression<UMaterialExpressionClamp>();
			Result->MinDefault = Min;
			Result->MaxDefault = Max;
			Result->Input.Connect(V.OutputIndex, V.Expression);

			return Box(Result);
		}

		FMaterialExpressionWrapper Floor(FMaterialExpressionWrapper V)
		{
			UMaterialExpressionFloor* Result = CreateMaterialExpression<UMaterialExpressionFloor>();
			Result->Input.Connect(V.OutputIndex, V.Expression);
			return Box(Result);
		}

		FMaterialExpressionWrapper Frac(FMaterialExpressionWrapper V)
		{
			UMaterialExpressionFrac* Result = CreateMaterialExpression<UMaterialExpressionFrac>();
			Result->Input.Connect(V.OutputIndex, V.Expression);
			return Box(Result);
		}

		FMaterialExpressionWrapper Normalize(FMaterialExpressionWrapper V)
		{
			UMaterialExpressionNormalize* Result = CreateMaterialExpression<UMaterialExpressionNormalize>();
			Result->VectorInput.Connect(V.OutputIndex, V.Expression);
			return Box(Result);
		}

		FMaterialExpressionWrapper Mul(FMaterialExpressionWrapper V, float B)
		{
			UMaterialExpressionMultiply* Product = CreateMaterialExpression<UMaterialExpressionMultiply>();
			Product->A.Connect(V.OutputIndex, V.Expression);
			Product->ConstB = B;
			return Box(Product);
		}

		FMaterialExpressionWrapper Mul(float A, FMaterialExpressionWrapper B)
		{
			UMaterialExpressionMultiply* Product = CreateMaterialExpression<UMaterialExpressionMultiply>();
			Product->ConstA = A;
			Product->B.Connect(B.OutputIndex, B.Expression);
			return Box(Product);
		}

		FMaterialExpressionWrapper Mul(FMaterialExpressionWrapper A, FMaterialExpressionWrapper B)
		{
			UMaterialExpressionMultiply* Product = CreateMaterialExpression<UMaterialExpressionMultiply>();
			Product->A.Connect(A.OutputIndex, A.Expression);
			Product->B.Connect(B.OutputIndex, B.Expression);
			return Box(Product);
		}

		FMaterialExpressionWrapper Div(FMaterialExpressionWrapper A, FMaterialExpressionWrapper B)
		{
			UMaterialExpressionDivide* Product = CreateMaterialExpression<UMaterialExpressionDivide>();
			Product->A.Connect(A.OutputIndex, A.Expression);
			Product->B.Connect(B.OutputIndex, B.Expression);
			return Box(Product);
		}

		FMaterialExpressionWrapper Acos(FMaterialExpressionWrapper V)
		{
			UMaterialExpressionArccosine* Result = CreateMaterialExpression<UMaterialExpressionArccosine>();
			Result->Input.Connect(V.OutputIndex, V.Expression);
			return Box(Result);
		}

		FMaterialExpressionWrapper Sqrt(FMaterialExpressionWrapper V)
		{
			UMaterialExpressionSquareRoot* Result = CreateMaterialExpression<UMaterialExpressionSquareRoot>();
			Result->Input.Connect(V.OutputIndex, V.Expression);
			return Box(Result);
		}


		FMaterialExpressionWrapper Fresnel(FMaterialExpressionWrapper BaseReflectFraction)
		{
			UMaterialExpressionFresnel* Result = CreateMaterialExpression<UMaterialExpressionFresnel>();
			Result->BaseReflectFractionIn.Connect(BaseReflectFraction.OutputIndex, BaseReflectFraction.Expression);
			return Box(Result);
		}

		FMaterialExpressionWrapper Append(FMaterialExpressionWrapper A, FMaterialExpressionWrapper B)
		{
			UMaterialExpressionAppendVector* Result = CreateMaterialExpression<UMaterialExpressionAppendVector>();
			Result->A.Connect(A.OutputIndex, A.Expression);
			Result->B.Connect(B.OutputIndex, B.Expression);
			return Box(Result);
		}

		FMaterialExpressionWrapper Mask(FMaterialExpressionWrapper BaseReflectFraction, bool R, bool G, bool B, bool A=false)
		{
			UMaterialExpressionComponentMask* Result = CreateMaterialExpression<UMaterialExpressionComponentMask>();
			Connect(Result->Input, BaseReflectFraction);
			Result->R = R;
			Result->G = G;
			Result->B = B;
			Result->A = A;

			return Box(Result);
		}
		
		FMaterialExpressionWrapper R(FMaterialExpressionWrapper V)
		{
			return Mask(V, true, false, false);
		}

		FMaterialExpressionWrapper G(FMaterialExpressionWrapper V)
		{
			return Mask(V, false, true, false);
		}

		FMaterialExpressionWrapper B(FMaterialExpressionWrapper V)
		{
			return Mask(V, false, false, true);
		}

		FMaterialExpressionWrapper Box(UMaterialExpression* Expression)
		{
			return FMaterialExpressionWrapper(Expression);
		}

		FMaterialExpressionWrapper Box(UMaterialExpression* Expression, int32 OutputIndex)
		{
			return FMaterialExpressionWrapper(Expression, OutputIndex);
		}

		UMaterialExpression* UnBox(FMaterialExpressionWrapper Wrapper)
		{
			check(0 == Wrapper.OutputIndex);
			return Wrapper.Expression;
		}

		UMaterialExpressionTextureSample* SampleTexture(EMaterialSamplerType SamplerType, UTexture2D* Texture, FMaterialExpressionWrapper Coordinate)
		{
			UMaterialExpressionTextureObject* ExpressionTextureObject = Cast<UMaterialExpressionTextureObject>(UMaterialEditingLibrary::CreateMaterialExpression(Material, UMaterialExpressionTextureObject::StaticClass()));
			ExpressionTextureObject->SamplerType = SamplerType;
			ExpressionTextureObject->Texture = Texture;

			UMaterialExpressionTextureSample* ExpressionTextureSample = Cast<UMaterialExpressionTextureSample>(UMaterialEditingLibrary::CreateMaterialExpression(Material, UMaterialExpressionTextureSample::StaticClass()));
			ExpressionTextureSample->TextureObject.Connect(0, ExpressionTextureObject);
			Connect(ExpressionTextureSample->Coordinates, Coordinate);

			return ExpressionTextureSample;
		}

		// Sample color Texture taking into account provided IntensityFactor in such way that color is not scaled too much(resulting in distortion/saturation after being clamped by 1  in the shader) 
		FMaterialExpressionWrapper SampleBaseColorTexture(FProcessedTextureSource Source, FMaterialExpressionWrapper Coordinate, float IntensityFactor)
		{
			if (Source.Texture)
			{
				auto Sampled = SampleTexture(Source.SamplerType, Source.Texture, Coordinate);
				if (Source.Scale != FVector::OneVector) 
				{
					// Use 'MaxColorFactorToAvoidClamping' factor instead of passed intensity in came MaxColorFactorToAvoidClamping is smaller:
					// - if Intensity(computed reflectance) is even smaller we won't loose(due to clamping) color information
					// - if Intensity is larger than MaxColorFactorToAvoidClamping this means that texture colors will be clamped after multiplied by Intensity and color will be altered
					// E.g. is we have BRFD color like (5, 2, 0.5) - strong red reflection, medium green, and (relatively)slight blue
					// Passing (5, 2, 0.5) to Diffuse Color will clamp this to (1, 1, 0.5) - a yellow-white color(instead of almost red original)
					// Ideally, modulating it by 1/5 will give (1, 0.4, 0.01) - same reddish color
					// This way, in case texture consists of colors that are mostly too reflective, we can have something closer to original material colors than just keeping (1, 1, 0.5)
					float Scale = FMath::Min(IntensityFactor, Source.MaxColorFactorToAvoidClamping);
					return Mul(Sampled, Constant(Source.Scale*Scale));
				}
				else 
				{
					return Mul(Sampled, Constant(IntensityFactor));
				}
			}
			return Constant(Source.Constant);
		}

		FMaterialExpressionWrapper SampleTexture(FProcessedTextureSource Source, FMaterialExpressionWrapper Coordinate)
		{
			if (Source.Texture)
			{
				auto Sampled = SampleTexture(Source.SamplerType, Source.Texture, Coordinate);
				return (Source.Scale != FVector::OneVector) ? Mul(Sampled, Constant(Source.Scale)) : Sampled;
			}
			return Constant(Source.Constant);
		}

		FMaterialExpressionWrapper SimpleTextureExpression(FProcessedTextureSource Source)
		{

			if (Source.Texture)
			{
				UMaterialExpressionTextureCoordinate* ExpressionTextureCoordinate = Cast<UMaterialExpressionTextureCoordinate>(UMaterialEditingLibrary::CreateMaterialExpression(Material, UMaterialExpressionTextureCoordinate::StaticClass()));
				ExpressionTextureCoordinate->CoordinateIndex = 0;
				ExpressionTextureCoordinate->UTiling = Source.Tiling.X;
				ExpressionTextureCoordinate->VTiling = Source.Tiling.Y;

				return SampleTexture(Source, ExpressionTextureCoordinate);
			}
			else
			{
				return Constant(Source.Constant);
			}
		}

		UMaterialExpressionConstant* Constant(float Value)
		{
			UMaterialExpressionConstant* Result = CreateMaterialExpression<UMaterialExpressionConstant>();
			Result->R = Value;
			return Result;
		}

		UMaterialExpressionConstant2Vector* Constant(float R, float G)
		{
			UMaterialExpressionConstant2Vector* Result = CreateMaterialExpression<UMaterialExpressionConstant2Vector>();
			Result->R = R;
			Result->G = G;
			return Result;
		}

		UMaterialExpressionConstant3Vector* Constant(float R, float G, float B)
		{
			UMaterialExpressionConstant3Vector* Result = CreateMaterialExpression<UMaterialExpressionConstant3Vector>();
			Result->Constant.R = R;
			Result->Constant.G = G;
			Result->Constant.B = B;
			return Result;
		}

		UMaterialExpressionConstant3Vector* Constant(FVector V)
		{
			return Constant(V.X, V.Y, V.Z);
		}

		UMaterialExpressionSquareRoot* Sqrt(UMaterialExpression* Value)
		{
			UMaterialExpressionSquareRoot* Result = CreateMaterialExpression<UMaterialExpressionSquareRoot>();
			Result->Input.Connect(0, Value);
			return Result;
		}

		template<class InputType>
		void Connect(InputType& Input, FMaterialExpressionWrapper V)
		{
			check(V.Expression);
			Input.Connect(V.OutputIndex, V.Expression);
		}

	protected:

		float CC_IOR = 1.0f;
		UTexture2D* FlakesThetaFISliceLUT;
		int FlakesNumThetaI;
		int FlakesNumThetaF;
		int FlakesMaxThetaI;
		float CT_Diffuse;
		TArray<float> CT_Spreads;
		TArray<float> CT_Coeffs;
		TArray<float> CT_F0s;
		bool bNoRefraction = true;
	};

	class FUeGgxBRDF
	{
	public:

		float Saturate(float V)
		{
			return FMath::Max(0.0f, FMath::Min(V, 1.0f));
		}

		// from BRDF.ush
		// GGX / Trowbridge-Reitz
		// [Walter et al. 2007, "Microfacet models for refraction through rough surfaces"]
		float D_GGX(float a2, float NoH)
		{
			float d = (NoH * a2 - NoH) * NoH + 1;	// 2 mad
			return a2 / (PI * d * d);					// 4 mul, 1 rcp
		}

		// Appoximation of joint Smith term for GGX
		// [Heitz 2014, "Understanding the Masking-Shadowing Function in Microfacet-Based BRDFs"]
		float Vis_SmithJointApprox(float a2, float NoV, float NoL)
		{
			float a = sqrt(a2);
			float Vis_SmithV = NoL * (NoV * (1 - a) + a);
			float Vis_SmithL = NoV * (NoL * (1 - a) + a);
			return 0.5f * 1.0f /(Vis_SmithV + Vis_SmithL);
		}

		// [Schlick 1994, "An Inexpensive BRDF Model for Physically-Based Rendering"]
		FVector F_Schlick(FVector SpecularColor, float VoH)
		{
			
			float Fc = FMath::Pow(1 - VoH, 5);					// 1 sub, 3 mul
			//return Fc + (1 - Fc) * SpecularColor;		// 1 add, 3 mad

			// Anything less than 2% is physically impossible and is instead considered to be shadowing
			return FVector(Saturate(50.0 * SpecularColor.Y) * Fc) + (1 - Fc) * SpecularColor;

		}

		// from ShadingModels.ush
		FVector SpecularGGX(float Roughness, FVector SpecularColor, float NoV, float NoL, float NoH, float VoH)
		{
			float a2 = FMath::Pow(Roughness, 4);
			//float Energy = EnergyNormalization(a2, Context.VoH, AreaLight);
			float Energy = 1.0f;

			// Generalized microfacet specular
			float D = D_GGX(a2, NoH) * Energy;
			float Vis = Vis_SmithJointApprox(a2, NoV, NoL);
			FVector F = F_Schlick(SpecularColor, VoH);

			return (D * Vis) * F;
		}

	};

	class FCarPaint2BRDF
	{
	public:
		struct FParams
		{
			float Diffuse;

			float Spreads[3];
			float Coeffs[3];
			float F0s[3];
		};

		float CT_F(float H_V, float F0)
		{
			return F0 + (1.0 - F0) * FMath::Pow(1.0 - H_V, 5);
		}

		// https://en.wikipedia.org/wiki/Specular_highlight#Beckmann_distribution
		float CT_D(float N_H, float m)
		{
			float CosSqr = FMath::Square(N_H);
			return FMath::Exp((CosSqr - 1.0) / (CosSqr * FMath::Square(m))) / (FMath::Square(m) * FMath::Square(CosSqr));
		}

		float CT_G(float NoV, float NoL, float NoH, float HoV)
		{
			return FMath::Min(1.0f, FMath::Min((2.0f * NoH * NoV / HoV), (2.0f * NoH * NoL / HoV)));
		}

		float CT(const FParams& params, int LobeCount, float NoV, float NoL, float NoH, float HoV)
		{
			float Result = params.Diffuse / PI;

			// specular term
			float Scale = CT_G(NoH, NoV, NoL, HoV) / (PI * NoV * NoL);
			for (int32 LobeIndex = 0; LobeIndex < LobeCount; ++LobeIndex)
			{
				float Spread = params.Spreads[LobeIndex];
				Result += params.Coeffs[LobeIndex] * CT_D(NoH, Spread) * CT_F(HoV, params.F0s[LobeIndex]) * Scale;
			}
			return Result;
		}

	};

	class FCreatedCarpaint2Material: public ICreatedMaterial
	{
	public:

		bool bIsAnisotropic;
		bool bHasFresnel;


		FMaterialExpressionWrapper ExpressionDiffuseColor = nullptr;
		FMaterialExpressionWrapper ExpressionSpecularColor = nullptr;
		FMaterialExpressionWrapper ExpressionFresnel = nullptr;

		FMaterialExpressionWrapper ExpressionNormal = nullptr;

		FMaterialExpressionWrapper ExpressionClearcoatIOR = nullptr;
		FMaterialExpressionWrapper ExpressionClearcoatNormal = nullptr;
		FMaterialExpressionWrapper ExpressionClearcoatColor = nullptr;

		bool bHasBRDFColorsTexture = false;
		FProcessedTextureSource BRDFColorsTexture;
		bool bHasBTFFlakes = false;
		FProcessedTextureSource BTFFlakes;

		bool bEnableClearcoat = true;

		void Build() override
		{
			FitBRDF();

			auto BaseColor = Box(nullptr);
			auto Specular = Box(nullptr);

			UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();

			if (ExpressionFresnel)
			{
				auto F0 = R(ExpressionFresnel);

				if (ExpressionDiffuseColor && ExpressionSpecularColor)
				{
					// approximate by shifting to Specular color as F0 approaches 1(what we map to Metallic)
					BaseColor = Add(ExpressionDiffuseColor, Mul(ExpressionSpecularColor, F0));
				}
				else
				{
					// set base color to one of present diffuse/specular colors
					if (ExpressionDiffuseColor)
					{
						BaseColor = ExpressionDiffuseColor;
					}

					if (ExpressionSpecularColor)
					{
						BaseColor = ExpressionSpecularColor;
					}
				}
				Connect(MaterialEditorOnly->Metallic, F0); // just to make sure that F0=1 switches Metallic

				auto ScaleFresnelToAccountForUESpecularScaling = Mul(Fresnel(F0), 1 / 0.08f); // UE scales Specular by 0.08 to compute F0
				Specular = ExpressionSpecularColor ? Mul(ScaleFresnelToAccountForUESpecularScaling, ExpressionSpecularColor) : ScaleFresnelToAccountForUESpecularScaling;
			}
			else
			{
				BaseColor = ExpressionDiffuseColor;

				if (ExpressionSpecularColor)
				{
					Specular = Mul(ExpressionSpecularColor, 1 / 0.08f); // UE scales Specular by 0.08 to compute F0
				}
			}

			Connect(MaterialEditorOnly->Metallic, Constant(1.0f));
			Connect(MaterialEditorOnly->Roughness, Constant(RoughnessFitted));

			if (bHasBRDFColorsTexture || bHasBTFFlakes)
			{
				UMaterialExpressionPixelNormalWS* N = CreateMaterialExpression<UMaterialExpressionPixelNormalWS>();
				UMaterialExpressionCameraVectorWS* V = CreateMaterialExpression<UMaterialExpressionCameraVectorWS>();

				// hardcoded light!
				UMaterialExpressionVectorParameter* LightDirection = CreateMaterialExpression<UMaterialExpressionVectorParameter>();
				LightDirection->DefaultValue = FLinearColor(0.f, 0.f, 1.f);
				LightDirection->ParameterName = FName(TEXT("Dominant Light Direction"));

				
				FMaterialExpressionWrapper L = LightDirection;

				if (!bNoRefraction)
				{
					// AxF-Decoding-SDK-1.5.1 "Hybrid Model- and Image-based Carpaint Representation"
					// Says that "incoming direction is a refracted direction"

					float IOR = CC_IOR;
					float eta = 1.0f / IOR;

					auto I = Mul(-1.0f, LightDirection);

					// https://www.khronos.org/registry/OpenGL-Refpages/gl4/html/refract.xhtml
					// k = 1.0 - eta * eta * (1.0 - dot(N, I) * dot(N, I));
					auto IoN = Dot(I, N);;
					auto k = Sub(1.0f, Mul(eta * eta, Sub(1.0f, Mul(IoN, IoN))));

					// R = eta * I - (eta * dot(N, I) + sqrt(k)) * N;
					L = Mul(-1.0f, Add(Mul(eta, I), Mul(-1.f, Mul(N, Add(Mul(eta, IoN), Sqrt(k))))));
				}
				

				auto H = Normalize(Add(L, V));

				auto NoH = Dot(N, H);
				auto LoH = Dot(L, H);

				// do we really need to clamp? N and H should be unit length already...
				auto ThetaF = Acos(Clamp(NoH, 0.f, 1.f));
				auto ThetaI = Acos(Clamp(LoH, 0.f, 1.f));


				if (bHasBRDFColorsTexture)
				{
					auto UV = Append(Mul(ThetaF, 1.f / HALF_PI), Mul(ThetaI, 1.f / HALF_PI));
					BaseColor = SampleBaseColorTexture(BRDFColorsTexture, UV, SpecularityFitted);
				}
				else
				{
					BaseColor = Constant(SpecularityFitted);
				}

				if (bHasBTFFlakes)
				{
					UMaterialExpressionTextureCoordinate* UV = CreateMaterialExpression<UMaterialExpressionTextureCoordinate>();
					UV->UTiling = BTFFlakes.Tiling.X;
					UV->VTiling = BTFFlakes.Tiling.Y;

					// NOTE: SDK example used FlakesNumThetaF for both ThetaI and ThetaF, for some reason?
					int AngularSampling = FlakesNumThetaF;
					auto ThetaFForLUT = Add(Mul(ThetaF, float(AngularSampling)  / HALF_PI), 0.5f);
					auto ThetaIForLUT = Add(Mul(ThetaI, float(AngularSampling)  / HALF_PI), 0.5f);

					//bilinear interp indices and weights
					auto ThetaFLow = Floor(ThetaFForLUT);
					auto ThetaFHigh = Add(ThetaFLow, 1);
					auto ThetaILow = Floor(ThetaIForLUT);
					auto ThetaIHigh = Add(ThetaILow, 1);

					auto ThetaFWeight = Sub(ThetaFForLUT, ThetaFLow);
					auto ThetaIWeight = Sub(ThetaIForLUT, ThetaILow);

					auto v2_offset_l = Constant(0, 0);
					auto v2_offset_h = Constant(0, 0);

					// Possible improvement:
                    // (from AxF SDK)
					// ...to allow lower thetaI samplings while preserving flake lifetime 
					//"virtual" thetaI patches are generated by shifting existing ones 
	//  				if (FlakesNumThetaI < AngularSampling)
	//  				{
	// 
	//  
	//  				}


					// NOTE: DOT'T optimize off Add(ThetaI_low, 1.f) into ThetaI_high
					// virtual samples above might changed ThetaI_low
					auto GetFlake = [this](FMaterialExpressionWrapper slice, FMaterialExpressionWrapper UV, FMaterialExpressionWrapper ThetaBase) {
						return If(slice, SampleFlakesSliceLUT(Add(ThetaBase, 1.f)),
							SampleFlake(UV, slice), Constant(0.f, 0.f, 0.f));
					};

					auto LUTLow = SampleFlakesSliceLUT(ThetaILow);
					auto LUTHi = SampleFlakesSliceLUT(ThetaIHigh);
// 
					auto ll = GetFlake(Add(LUTLow, ThetaFLow), Add(UV, v2_offset_l), ThetaILow);
					auto hl = GetFlake(Add(LUTLow, ThetaFHigh), Add(UV, v2_offset_l), ThetaILow);
					auto lh = GetFlake(Add(LUTHi, ThetaFLow), Add(UV, v2_offset_h), ThetaIHigh);
					auto hh = GetFlake(Add(LUTHi, ThetaFHigh), Add(UV, v2_offset_h), ThetaIHigh);

					auto FlakeResult = Lerp(
						Lerp(ll, hl, ThetaFWeight),
						Lerp(lh, hh, ThetaFWeight),
						ThetaIWeight);

					Connect(MaterialEditorOnly->EmissiveColor, FlakeResult);
				}
			}

			if (BaseColor)
			{
				Connect(MaterialEditorOnly->BaseColor, BaseColor);
			}

			if (Specular)
			{
				Connect(MaterialEditorOnly->Specular, Specular);
			}

			bool bHasClearcoat = ExpressionClearcoatIOR || ExpressionClearcoatNormal || ExpressionClearcoatColor || (CC_IOR != 1.0f);
			// seems like AxF decoding sdk doesn't provide any other clues when material has ClearCoat layer
			if (bHasClearcoat)
			{
				Material->SetShadingModel(MSM_ClearCoat);
			}

			if (bHasClearcoat)
			{
				// Clearcoat normal goes to regular Normal input
				if (ExpressionClearcoatNormal)
				{
					Connect(MaterialEditorOnly->Normal, ExpressionClearcoatNormal);
				}

				if (ExpressionNormal)
				{
					UMaterialExpressionClearCoatNormalCustomOutput* BottomNormal = CreateMaterialExpression< UMaterialExpressionClearCoatNormalCustomOutput>();
					Connect(BottomNormal->Input, ExpressionNormal);
				}

				Connect(MaterialEditorOnly->ClearCoatRoughness, Constant(0.0f));
			}
			else
			{
				if (ExpressionNormal)
				{
					Connect(MaterialEditorOnly->Normal, ExpressionNormal);
				}
			}
		}

		float RoughnessFitted;
		float SpecularityFitted;

		// Finds UE GGX shading model parameters(roughness and Specularity) which fit best
		// source AxF material's BRDF(X-Rite CPA2)
		void FitBRDF()
		{
			// BRDFs are parametrized by angles between light/normal(ThetaL) and view/normal(ThetaV) over hemisphere around surface normal.
			const int32 LVGridSize = 100; 
			const int32 RougnessSpecularitySearchAreaSize = 30; // Search begins with whole range of roughness/specularity using this resolution
			const float IterationRangeReduction = 0.2f; // After finding best fit search restarts around found value in a smaller area
			const int32 IterationCount = 5;
			const float RayAngleExtent = HALF_PI * 0.8f; // don't take into account angles too close to grazing - spikes are too big there 
			const bool bUseL2Distance = true;

			static float VisibleDirections[LVGridSize];

			// use light/view directions around normal from -pi/2 to pi/2
			for (int I = 0; I < LVGridSize; ++I)
			{
				float Alpha = float(I) / (LVGridSize - 1);
				VisibleDirections[I] = -RayAngleExtent * (1.0f - Alpha) + RayAngleExtent * Alpha;
			}

			static float ThetaL[LVGridSize][LVGridSize];
			static float ThetaV[LVGridSize][LVGridSize];
			static float ThetaH[LVGridSize][LVGridSize];

			static float N_V[LVGridSize][LVGridSize];
			static float N_H[LVGridSize][LVGridSize];
			static float N_L[LVGridSize][LVGridSize];
			static float H_V[LVGridSize][LVGridSize];

			for (int U = 0; U < LVGridSize; ++U)
			{
				for (int V = 0; V < LVGridSize; ++V)
				{
					ThetaL[U][V] = VisibleDirections[U];
					ThetaV[U][V] = VisibleDirections[V];
					ThetaH[U][V] = (ThetaL[U][V] + ThetaV[U][V]) * 0.5f; // half-way vector(in the same plane) 

					N_L[U][V] = FMath::Cos(ThetaL[U][V]);
					N_V[U][V] = FMath::Cos(ThetaV[U][V]);
					N_H[U][V] = FMath::Cos(ThetaH[U][V]);
					H_V[U][V] = FMath::Cos(ThetaH[U][V] - ThetaV[U][V]);
				}
			}

			FCarPaint2BRDF CarPaint2BRDF;
			FUeGgxBRDF UeGgxBRDF;

			FCarPaint2BRDF::FParams Params;
			Params.Diffuse = CT_Diffuse;
			for (int32 Lobe = 0; Lobe < 3; ++Lobe)
			{
				Params.Coeffs[Lobe] = Lobe < CT_Coeffs.Num() ? CT_Coeffs[Lobe] : 0.0f;
				Params.Spreads[Lobe] = Lobe < CT_Spreads.Num() ? CT_Spreads[Lobe] : 1.0f;
				Params.F0s[Lobe] = Lobe < CT_F0s.Num() ? CT_F0s[Lobe] : 1.0f;
			}

			float RoughnessRange[2] = { 0.01f, 1.0f };
			float SpecularityRange[2] = { 0.0f, 1.0f };

			float ClosestDistance = BIG_NUMBER;
			float ClosestRoughness = 1.0f;
			float ClosestSpecularity = 1.0f;
			for (int Iteration = 0; Iteration < IterationCount; ++Iteration)
			{

				for (int32 RoughnessIndex = 0; RoughnessIndex < RougnessSpecularitySearchAreaSize; ++RoughnessIndex)
				{
					float Specularities[RougnessSpecularitySearchAreaSize];
					float DistanceForSpecularity[RougnessSpecularitySearchAreaSize];

					float Roughness = FMath::Lerp(RoughnessRange[0], RoughnessRange[1], float(RoughnessIndex) / (RougnessSpecularitySearchAreaSize-1));

					ParallelFor(RougnessSpecularitySearchAreaSize,
						[&](int32 SpecularityIndex)
						{
							float Specularity = FMath::Lerp(SpecularityRange[0], SpecularityRange[1], float(SpecularityIndex) / (RougnessSpecularitySearchAreaSize-1));

							float SumOfSquaredDiff = 0.0f;
							float SumOfAbsDiff = 0.0f;
							float SumOfSqrtDiff = 0.0f;

							for (int U = 0; U < LVGridSize; ++U)
							{
								for (int V = 0; V < LVGridSize; ++V)
								{
									float NoL = N_L[U][V];
									float NoV = N_V[U][V];
									float NoH = N_H[U][V];
									float HoV = H_V[U][V];

									float xrite = NoL*CarPaint2BRDF.CT(Params, 3, NoV, NoL, NoH, HoV);
									FVector3f ue = FVector3f(NoL*UeGgxBRDF.SpecularGGX(Roughness, FVector(Specularity), NoV, NoL, NoH, HoV));

									SumOfSquaredDiff += FMath::Square(FMath::Max(xrite, 0.0f) - FMath::Max(ue.X, 0.0f));
									SumOfAbsDiff += FMath::Abs(FMath::Max(xrite, 0.0f) - FMath::Max(ue.X, 0.0f));
									SumOfSqrtDiff += FMath::Sqrt(FMath::Abs(FMath::Max(xrite, 0.0f) - FMath::Max(ue.X, 0.0f)));
								}
							}

							//float Distance = SumOfSquaredDiff;
							//float Distance = SumOfAbsDiff;
							float Distance = SumOfAbsDiff;

							Specularities[SpecularityIndex] = Specularity;
							DistanceForSpecularity[SpecularityIndex] = Distance;
						}
					);

					for (int32 SpecularityIndex = 0; SpecularityIndex < RougnessSpecularitySearchAreaSize; ++SpecularityIndex)
					{
						float Specularity = Specularities[SpecularityIndex];
						float Distance = DistanceForSpecularity[SpecularityIndex];
						if (Distance < ClosestDistance)
						{
						 	ClosestDistance = Distance;
						 	ClosestRoughness = Roughness;
						 	ClosestSpecularity = Specularity;
						}
					}
				}

				// reduce search ranges
				float RoughnessSpread = (RoughnessRange[1] - RoughnessRange[0]) * IterationRangeReduction;
				RoughnessRange[0] = FMath::Max(0.0f, ClosestRoughness - RoughnessSpread);
				RoughnessRange[1] = FMath::Min(1.0f, ClosestRoughness + RoughnessSpread);

				float SpecularitySpread = (SpecularityRange[1] - SpecularityRange[0]) * IterationRangeReduction;
				SpecularityRange[0] = FMath::Max(0.0f, ClosestSpecularity - SpecularitySpread);
				SpecularityRange[1] = FMath::Min(1.0f, ClosestSpecularity + SpecularitySpread);
			}

			RoughnessFitted = ClosestRoughness;
			SpecularityFitted = ClosestSpecularity;
		}

		FMaterialExpressionWrapper SampleFlake(FMaterialExpressionWrapper ll_uv, FMaterialExpressionWrapper ll_slice)
		{
// 			CreateFunction(TEXT("SampleFlakeTexture"), [this](UMaterialFunction* Function) 
// 				{
// 					UMaterialExpressionFunctionOutput* Expression = CreateMaterialExpression<UMaterialExpressionFunctionOutput>();
// 				});

			// sample from flake atlas, where slices are placed row-by-row
			float FlakesTextureSlicesPerRow = FlakesTextureAtlas.FlakesTextureSlicesPerRow;
			float TargetWidth = FlakesTextureAtlas.TextureWidth;
			float TargetHeight = FlakesTextureAtlas.TextureHeight;
			float TargetSliceWidth = FlakesTextureAtlas.SliceWidth;
			float TargetSliceHeight = FlakesTextureAtlas.SliceHeight;


			auto SliceV = Floor(Mul(ll_slice, 1.f / FlakesTextureSlicesPerRow));
			auto SliceU = Sub(ll_slice, Mul(SliceV, FlakesTextureSlicesPerRow));

			auto FlakesUV = Mul(
				Add(Append(SliceU, SliceV), Frac(ll_uv)),
				Constant(TargetSliceWidth / TargetWidth, TargetSliceHeight / TargetHeight));

			auto FlakeSampled = SampleTexture(BTFFlakes, FlakesUV);
			return (BTFFlakes.Scale != FVector::OneVector) ? Mul(FlakeSampled, Constant(BTFFlakes.Scale)) : FlakeSampled;
		}

		FMaterialExpressionWrapper SampleFlakesSliceLUT(FMaterialExpressionWrapper ThetaI_low)
		{
			// normalize Theta into UV coordinates, we don't use array(indexed with int) as in SDK sample
			// add half texel to make sure our integer Theta is inside the texel
			auto UV = Add(Div(ThetaI_low, Constant((float)FlakesThetaFISliceLUT->GetSizeX())), Constant(0.5/ FlakesThetaFISliceLUT->GetSizeX()));
			return Mul(Box(SampleTexture(SAMPLERTYPE_Alpha, FlakesThetaFISliceLUT, UV), 1), 255.f);
		}

		void SetTextureDiffuseColor(FProcessedTextureSource Source) override
		{
			ExpressionDiffuseColor = SimpleTextureExpression(Source);
		}

		void SetTextureAlpha(FProcessedTextureSource Source) override
		{
		}

		void SetTextureHeight(FProcessedTextureSource Source) override
		{
		}

		void SetTextureNormal(FProcessedTextureSource Source) override
		{
			ExpressionNormal = SimpleTextureExpression(Source);
		}

		void SetTextureSpecularColor(FProcessedTextureSource Source) override
		{
			ExpressionSpecularColor = SimpleTextureExpression(Source);
		}

		void SetTextureFresnel(FProcessedTextureSource Source) override
		{
			ExpressionFresnel = SimpleTextureExpression(Source);
		}

		void SetTextureSpecularLobe(FProcessedTextureSource Source) override
		{
			// alpha for isotropic or (alpha, beta) for anisotropic
			FMaterialExpressionWrapper SpecularLobes = SimpleTextureExpression(Source);

			auto SpecularRoughness = Box(nullptr);

			if (bIsAnisotropic)
			{
				auto LobeU = R(SpecularLobes);
				auto LobeV = G(SpecularLobes);
				SpecularRoughness = Mul(Add(LobeU, LobeV), 0.5f);
			}
			else
			{
				SpecularRoughness = R(SpecularLobes);
			}

			// UE squares UI roughness, so take root
			Connect(Material->GetEditorOnlyData()->Roughness, Sqrt(SpecularRoughness));
		}

		void SetIsAnisotropic(bool InIsAnisotropic) override
		{
			bIsAnisotropic = InIsAnisotropic;
		}

		void SetHasFresnel(bool InHasFresnel) override
		{
			bHasFresnel = InHasFresnel;
		}

		void SetTextureClearcoatColor(FProcessedTextureSource Source) override
		{
			if (!bEnableClearcoat)
			{
				return;
			}
			ExpressionClearcoatColor = SimpleTextureExpression(Source);
		}

		void SetTextureClearcoatIOR(FProcessedTextureSource Source) override
		{
			if (!bEnableClearcoat)
			{
				return;
			}
			ExpressionClearcoatIOR = SimpleTextureExpression(Source);
		}

		void SetTextureClearcoatNormal(FProcessedTextureSource Source) override
		{
			if (!bEnableClearcoat)
			{
				return;
			}
			ExpressionClearcoatNormal = SimpleTextureExpression(Source);
		}

		void SetTextureCarpaint2BRDFColors(FProcessedTextureSource Source) override
		{
			BRDFColorsTexture = Source;
			bHasBRDFColorsTexture = true;
		}

		void SetTextureCarpaint2BTFFlakes(FProcessedTextureSource Source, FFlakesTextureAtlas Atlas) override
		{
			bHasBTFFlakes = true;
			BTFFlakes = Source;
			FlakesTextureAtlas = Atlas;
		}

		void SetTextureCarpaintClearcoatNormal(FProcessedTextureSource Source) override
		{
			ExpressionClearcoatNormal = SimpleTextureExpression(Source);
		}
private:
	FFlakesTextureAtlas FlakesTextureAtlas;
	};

	class FCreatedMaterial : public ICreatedMaterial
	{
	public:

		bool bIsAnisotropic;
		bool bHasFresnel;

		FMaterialExpressionWrapper ExpressionDiffuseColor = nullptr;

		FMaterialExpressionWrapper ExpressionAlpha;
		FMaterialExpressionWrapper ExpressionHeight;

		FMaterialExpressionWrapper ExpressionSpecularColor = nullptr;
		FMaterialExpressionWrapper ExpressionFresnel = nullptr;

		FMaterialExpressionWrapper ExpressionNormal = nullptr;

		FMaterialExpressionWrapper ExpressionClearcoatIOR = nullptr;
		FMaterialExpressionWrapper ExpressionClearcoatColor = nullptr;
		FMaterialExpressionWrapper ExpressionClearcoatNormal = nullptr;

		bool bEnableClearcoat = true;

		void Build()
		{
			auto BaseColor = Box(nullptr);
			auto Specular = Box(nullptr);

			UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();

			if (ExpressionFresnel)
			{
				auto F0 = R(ExpressionFresnel);

				// approximate by shifting to Specular color as F0 approaches 1(what we map to Metallic)
				if (ExpressionDiffuseColor && ExpressionSpecularColor)
				{
					BaseColor = Add(ExpressionDiffuseColor, Mul(ExpressionSpecularColor, F0));
				}
				else
				{
					// set base color to one of present diffuse/specular colors
					if (ExpressionDiffuseColor)
					{
						BaseColor = ExpressionDiffuseColor;
					}

					if (ExpressionSpecularColor)
					{
						BaseColor = ExpressionSpecularColor;
					}
				}
				Connect(MaterialEditorOnly->Metallic, F0); // just to make sure that F0=1 switches Metallic

				auto ScaleFresnelToAccountForUESpecularScaling = Mul(Fresnel(F0), 1 / 0.08f); // UE scales Specular by 0.08 to compute F0
				Specular = ExpressionSpecularColor ? Mul(ScaleFresnelToAccountForUESpecularScaling, ExpressionSpecularColor) : ScaleFresnelToAccountForUESpecularScaling;
			}
			else
			{
				BaseColor = ExpressionDiffuseColor;

				if (ExpressionSpecularColor)
				{
					Specular = Mul(ExpressionSpecularColor, 1 / 0.08f); // UE scales Specular by 0.08 to compute F0
				}
			}

			if (BaseColor)
			{
				Connect(MaterialEditorOnly->BaseColor, BaseColor);
			}

			if (Specular)
			{
				Connect(MaterialEditorOnly->Specular, Specular);
			}

			bool bHasClearcoat = ExpressionClearcoatIOR || ExpressionClearcoatNormal || ExpressionClearcoatColor;

			if (bHasClearcoat)
			{
				Material->SetShadingModel(MSM_ClearCoat);

				// Clearcoat normal goes to regular Normal input
				if (ExpressionClearcoatNormal)
				{
					Connect(MaterialEditorOnly->Normal, ExpressionClearcoatNormal);
				}

				if (ExpressionNormal)
				{
					UMaterialExpressionClearCoatNormalCustomOutput* BottomNormal = CreateMaterialExpression< UMaterialExpressionClearCoatNormalCustomOutput>();
					Connect(BottomNormal->Input, ExpressionNormal);
				}
			}
			else
			{
				if (ExpressionNormal)
				{
					Connect(MaterialEditorOnly->Normal, ExpressionNormal);
				}
			}

			if (ExpressionAlpha)
			{
				Material->BlendMode = EBlendMode::BLEND_Masked;
				Material->TwoSided = true;

				// MaterialFunction'/Engine/Functions/Engine_MaterialFunctions02/Utility/DitherTemporalAA.DitherTemporalAA'
				FString UtilityPath = TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility");
				FString FunctionName = TEXT("DitherTemporalAA.DitherTemporalAA");
				UMaterialFunction* DitherTemporalAAFunction = LoadObject<UMaterialFunction>(nullptr, *(UtilityPath / FunctionName), nullptr, LOAD_EditorOnly | LOAD_NoWarn | LOAD_Quiet, nullptr);

				if (DitherTemporalAAFunction)
				{
					UMaterialExpressionMaterialFunctionCall* DitherTemporalAA = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>();
					DitherTemporalAA->SetMaterialFunction(DitherTemporalAAFunction);

					Connect(DitherTemporalAA->FunctionInputs[0].Input, ExpressionAlpha);

					Connect(MaterialEditorOnly->OpacityMask, DitherTemporalAA);
				}
			}
		}

		void SetTextureDiffuseColor(FProcessedTextureSource Source) override
		{
			ExpressionDiffuseColor = SimpleTextureExpression(Source);
		}

		void SetTextureAlpha(FProcessedTextureSource Source) override
		{
			ExpressionAlpha = SimpleTextureExpression(Source);
		}

		void SetTextureHeight(FProcessedTextureSource Source) override
		{
			ExpressionHeight = Box(UnBox(SimpleTextureExpression(Source)), 1); // take R output
		}

		void SetTextureNormal(FProcessedTextureSource Source) override
		{
			ExpressionNormal = SimpleTextureExpression(Source);
		}

		void SetTextureSpecularColor(FProcessedTextureSource Source) override
		{
			ExpressionSpecularColor = SimpleTextureExpression(Source);
		}

		void SetTextureFresnel(FProcessedTextureSource Source) override
		{
			ExpressionFresnel = SimpleTextureExpression(Source);
		}

		void SetTextureSpecularLobe(FProcessedTextureSource Source) override
		{
			// alpha for isotropic or (alpha, beta) for anisotropic
			FMaterialExpressionWrapper SpecularLobes = SimpleTextureExpression(Source);

			auto SpecularRoughness = Box(nullptr);

			if (bIsAnisotropic)
			{
				auto LobeU = R(SpecularLobes);
				auto LobeV = G(SpecularLobes);
				SpecularRoughness = Mul(Add(LobeU, LobeV), 0.5f);
			}
			else
			{
				SpecularRoughness = R(SpecularLobes);
			}

			// UE squares UI roughness, so take root
			Connect(Material->GetEditorOnlyData()->Roughness, Sqrt(SpecularRoughness));
		}

		void SetIsAnisotropic(bool InIsAnisotropic)
		{
			bIsAnisotropic = InIsAnisotropic;
		}

		void SetHasFresnel(bool InHasFresnel)
		{
			bHasFresnel = InHasFresnel;
		}

		void SetTextureClearcoatColor(FProcessedTextureSource Source) override
		{
			if (!bEnableClearcoat)
			{
				return;
			}
			ExpressionClearcoatColor = SimpleTextureExpression(Source);
		}

		void SetTextureClearcoatIOR(FProcessedTextureSource Source) override
		{
			if (!bEnableClearcoat)
			{
				return;
			}
			ExpressionClearcoatIOR = SimpleTextureExpression(Source);
		}

		void SetTextureClearcoatNormal(FProcessedTextureSource Source) override
		{
			if (!bEnableClearcoat)
			{
				return;
			}
			ExpressionClearcoatNormal = SimpleTextureExpression(Source);
		}

		void SetTextureCarpaint2BRDFColors(FProcessedTextureSource Texture) override
		{

		}

		void SetTextureCarpaint2BTFFlakes(FProcessedTextureSource, FFlakesTextureAtlas) override
		{

		}

		void SetTextureCarpaintClearcoatNormal(FProcessedTextureSource) override
		{
		}
	};

	ICreatedMaterial& AddMaterial(FString MaterialName)
	{
		ICreatedMaterial& Material = *CreatedMaterials.Emplace_GetRef(new FCreatedMaterial).Get();
		Material.SetName(MaterialName);
		return Material;
	}

	ICreatedMaterial& AddCarPaint2Material(FString MaterialName)
	{
		ICreatedMaterial& Material = *CreatedMaterials.Emplace_GetRef(new FCreatedCarpaint2Material).Get();
		Material.SetName(MaterialName);
		return Material;
	}

	FAxFFileImporter():
		FileHandle(nullptr)
	{
	}

	~FAxFFileImporter()
	{
		if (FileHandle)
		{
			axf::decoding::axfCloseFile(&FileHandle);
		}
	}

	bool OpenFile(const FString& InFileName, const UAxFImporterOptions& InImporterOptions)
	{
		Filename = InFileName;
		Log.Messages.Empty();

#ifdef _WIN32
		FileHandle = axf::decoding::axfOpenFileW(*Filename, true);
#else
		FileHandle = axf::decoding::axfOpenFile(TCHAR_TO_ANSI(*Filename), true);
#endif
		if (!FileHandle)
		{
			Log.Error(TEXT("Can't open file: ") + Filename);
			return false;
		}

		MetersPerUVUnit = InImporterOptions.MetersPerSceneUnit;

		MaterialCount = axf::decoding::axfGetNumberOfMaterials(FileHandle);
		for (int32 MaterialIndex = 0; MaterialIndex!=MaterialCount; ++MaterialIndex)
		{
			axf::decoding::AXF_MATERIAL_HANDLE MaterialHandle = axf::decoding::axfGetMaterial(FileHandle, MaterialIndex);
			if (!MaterialHandle)
			{
				Log.Error(TEXT("Can't retrieve AxF material with ID: ") + FString::FromInt(MaterialIndex));
				continue;
			}

			FString MaterialName;
			if (int NameBufferSize = axf::decoding::axfGetMaterialDisplayName(MaterialHandle, nullptr, 0))
			{
				TArray<wchar_t> NameBuffer;
				NameBuffer.SetNumUninitialized(NameBufferSize);
				axf::decoding::axfGetMaterialDisplayName(MaterialHandle, NameBuffer.GetData(), NameBuffer.Num());
				MaterialName = WCHAR_TO_TCHAR(NameBuffer.GetData());
			}

			Log.Info(TEXT("Material name: ") + MaterialName);

			// Look for using axfGetBestCompatibleRepresentation
			axf::decoding::AXF_REPRESENTATION_HANDLE RepresentationHandle = axf::decoding::axfGetBestSDKSupportedRepresentation(MaterialHandle);
			if (!RepresentationHandle)
			{
				Log.Error(FString::Printf(TEXT("Can't retrieve preferred representation for material %d('%s')"), MaterialIndex, *MaterialName));
				continue;
			}

			char RepresentationClassBuffer[axf::decoding::AXF_MAX_KEY_SIZE];
			if (!axf::decoding::axfGetRepresentationClass(RepresentationHandle, RepresentationClassBuffer, axf::decoding::AXF_MAX_KEY_SIZE))
			{
				Log.Error(FString::Printf(TEXT("Can't retrieve preferred representation's class for material %d('%s')"), MaterialIndex, *MaterialName));
				continue;
			}

			FString RepresentationClassName(ANSI_TO_TCHAR(RepresentationClassBuffer));
			Log.Info(FString::Printf(TEXT("Preferred representation class '%s'"), *RepresentationClassName));

			ICreatedMaterial& Material = AddCreatedMaterial(RepresentationClassName, MaterialName);

			Material.SetRepresentationHandle(RepresentationHandle);
			Material.SetRepresentationClassName(RepresentationClassName);

			axf::decoding::AXF_REPRESENTATION_HANDLE DiffuseRepresentationHandle = axf::decoding::axfGetSvbrdfDiffuseModelRepresentation(RepresentationHandle);

			if (DiffuseRepresentationHandle)
			{
				char DiffuseTypeBuffer[axf::decoding::AXF_MAX_KEY_SIZE];
				if (!axf::decoding::axfGetRepresentationTypeKey(DiffuseRepresentationHandle, DiffuseTypeBuffer, axf::decoding::AXF_MAX_KEY_SIZE))
				{
				}
				FString DiffuseTypeName(ANSI_TO_TCHAR(DiffuseTypeBuffer));
				Log.Info(FString::Printf(TEXT("Diffuse representation of type '%s'"), *DiffuseTypeName));
			}

			axf::decoding::AXF_REPRESENTATION_HANDLE SpecularRepresentationHandle = axf::decoding::axfGetSvbrdfSpecularModelRepresentation(RepresentationHandle);

			if (SpecularRepresentationHandle)
			{
				char SpecularTypeBuffer[axf::decoding::AXF_MAX_KEY_SIZE];
				if (!axf::decoding::axfGetRepresentationTypeKey(SpecularRepresentationHandle, SpecularTypeBuffer, axf::decoding::AXF_MAX_KEY_SIZE))
				{
				}
				FString SpecularTypeName(ANSI_TO_TCHAR(SpecularTypeBuffer));

				char SpecularVariantBuffer[axf::decoding::AXF_MAX_KEY_SIZE];
				bool bIsAnisotropic = false;
				bool bHasFresnel = false;
				if (!axf::decoding::axfGetSvbrdfSpecularModelVariant(SpecularRepresentationHandle, SpecularVariantBuffer, axf::decoding::AXF_MAX_KEY_SIZE, bIsAnisotropic, bHasFresnel))
				{
				}
				Material.SetIsAnisotropic(bIsAnisotropic);
				Material.SetHasFresnel(bHasFresnel);

				FString SpecularVariantName(ANSI_TO_TCHAR(SpecularVariantBuffer));

				Log.Info(FString::Printf(TEXT("Specular representation of type '%s', variant '%s', %s, %s"),
					*SpecularTypeName, *SpecularVariantName, bIsAnisotropic ? TEXT("anisotropic") : TEXT("isotropic"), bHasFresnel ? TEXT("with fresnel") : TEXT("no fresnel")));

				if (bHasFresnel)
				{
					char SpecularFresnelVariantBuffer[axf::decoding::AXF_MAX_KEY_SIZE];
					if (!axf::decoding::axfGetSvbrdfSpecularFresnelVariant(SpecularRepresentationHandle, SpecularFresnelVariantBuffer, axf::decoding::AXF_MAX_KEY_SIZE))
					{
					}
					FString SpecularFresnelVariantName(ANSI_TO_TCHAR(SpecularFresnelVariantBuffer));
					Log.Info(FString::Printf(TEXT("    fresnel variant '%s'"), *SpecularFresnelVariantName));
				}
			}

		}

		return true;
	}

	ICreatedMaterial& AddCreatedMaterial(FString RepresentationClassName, FString MaterialName)
	{
		if (RepresentationClassName == AXF_REPRESENTATION_CLASS_SVBRDF)
		{
			return AddMaterial(MaterialName);
		}
		else if (RepresentationClassName == AXF_REPRESENTATION_CLASS_CARPAINT2)
		{
			return AddCarPaint2Material(MaterialName);
		}

		return AddMaterial(MaterialName);
	}

	int GetMaterialCountInFile() 
	{
		return MaterialCount;
	}

	bool ImportMaterials(UObject* ParentPackage, EObjectFlags Flags, FProgressFunc ProgressFunc /*= nullptr*/)
	{
		for (auto& MaterialPtr : CreatedMaterials)
		{
			ICreatedMaterial& Material = *MaterialPtr.Get();

			FString MaterialName =  ObjectTools::SanitizeObjectName(Material.Name);

			Log.Info(TEXT("Importing material: ") + MaterialName);

			FString  MaterialPackageName = UPackageTools::SanitizePackageName(*(ParentPackage->GetName() / MaterialName));
			UObject* MaterialPackage = CreatePackage(*MaterialPackageName);

			UMaterial* NewMaterial = NewObject<UMaterial>(MaterialPackage, UMaterial::StaticClass(), FName(*MaterialName), Flags);
			check(NewMaterial != nullptr);

			ImportMaterial(Material, NewMaterial, ParentPackage);
			Log.Info(TEXT("Done importing material: ") + MaterialName);
		}
		return true;
	}

	bool ImportMaterial(ICreatedMaterial& Material, UMaterial* NewMaterial, UObject* ParentPackage)
	{
		{
			Material.SetMaterialIntefrace(NewMaterial);

			axf::decoding::TextureDecoder* TextureDecoder = axf::decoding::TextureDecoder::create(Material.GetRepresentationHandle(), AXF_COLORSPACE_LINEAR_SRGB_E, axf::decoding::ORIGIN_TOPLEFT);
			if (TextureDecoder)
			{
				int PropertyCount = TextureDecoder->getNumProperties();

				for (int PropertyIndex = 0; PropertyIndex < PropertyCount; PropertyIndex++)
				{
					char PropertyNameBuffer[axf::decoding::AXF_MAX_KEY_SIZE];
					TextureDecoder->getPropertyName(PropertyIndex, PropertyNameBuffer, axf::decoding::AXF_MAX_KEY_SIZE);
					FString PropertyName(ANSI_TO_TCHAR(PropertyNameBuffer));


					int PropertyType = TextureDecoder->getPropertyType(PropertyIndex);
					int PropertySize = TextureDecoder->getPropertySize(PropertyIndex);

					if (PropertyName == TEXT(AXF_CLEARCOAT_PROPERTY_NAME_NO_REFRACTION))
					{
						if ((PropertyType == axf::decoding::TYPE_INT) && (4 == PropertySize))
						{
							int32 PropertyValue;
							if (TextureDecoder->getProperty(PropertyIndex, &PropertyValue, PropertyType, PropertySize))
							{
								Material.SetNoRefraction((bool)PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_BRDF_CT_F0S))
					{
						if (PropertyType == axf::decoding::TYPE_FLOAT_ARRAY)
						{
							TArray<float> PropertyValue; 
							PropertyValue.SetNumUninitialized(PropertySize / sizeof(float));
							if (TextureDecoder->getProperty(PropertyIndex, PropertyValue.GetData(), PropertyType, PropertySize))
							{
								Material.SetCT_F0s(PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_BRDF_CT_COEFFS))
					{
						if (PropertyType == axf::decoding::TYPE_FLOAT_ARRAY)
						{
							TArray<float> PropertyValue;
							PropertyValue.SetNumUninitialized(PropertySize / sizeof(float));
							if (TextureDecoder->getProperty(PropertyIndex, PropertyValue.GetData(), PropertyType, PropertySize))
							{
								Material.SetCT_Coeffs(PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_BRDF_CT_SPREADS))
					{
						if (PropertyType == axf::decoding::TYPE_FLOAT_ARRAY)
						{
							TArray<float> PropertyValue;
							PropertyValue.SetNumUninitialized(PropertySize / sizeof(float));
							if (TextureDecoder->getProperty(PropertyIndex, PropertyValue.GetData(), PropertyType, PropertySize))
							{
								Material.SetCT_Spreads(PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_BRDF_CT_DIFFUSE))
					{
						if ((PropertyType == axf::decoding::TYPE_FLOAT) && (sizeof(float) == PropertySize))
						{
							float PropertyValue;
							if (TextureDecoder->getProperty(PropertyIndex, &PropertyValue, PropertyType, PropertySize))
							{
								Material.SetCT_Diffuse(PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_CC_IOR))
					{
						if ((PropertyType == axf::decoding::TYPE_FLOAT) && (sizeof(float) == PropertySize))
						{
							float PropertyValue;
							if(TextureDecoder->getProperty(PropertyIndex, &PropertyValue, PropertyType, PropertySize))
							{
								Material.SetCC_IOR(PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_FLAKES_MAX_THETAI))
					{
						if ((PropertyType == axf::decoding::TYPE_INT) && (4 == PropertySize))
						{
							int32 PropertyValue;
							if (TextureDecoder->getProperty(PropertyIndex, &PropertyValue, PropertyType, PropertySize))
							{
								Material.SetFlakesMaxThetaI(PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_FLAKES_NUM_THETAF))
					{
						if ((PropertyType == axf::decoding::TYPE_INT) && (4 == PropertySize))
						{
							int32 PropertyValue;
							if (TextureDecoder->getProperty(PropertyIndex, &PropertyValue, PropertyType, PropertySize))
							{
								Material.SetFlakesNumThetaF(PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_FLAKES_NUM_THETAI))
					{
						if ((PropertyType == axf::decoding::TYPE_INT) && (4 == PropertySize))
						{
							int32 PropertyValue;
							if (TextureDecoder->getProperty(PropertyIndex, &PropertyValue, PropertyType, PropertySize))
							{
								Material.SetFlakesNumThetaI(PropertyValue);
							}
						}
					}
					else if (PropertyName == TEXT(AXF_CARPAINT2_PROPERTY_FLAKES_THETAFI_SLICE_LUT))
					{
						if (PropertyType == axf::decoding::TYPE_INT_ARRAY)
						{
							int32 Count = PropertySize / sizeof(int32);
							TArray<int32> PropertyValue;
							PropertyValue.SetNumUninitialized(Count);
							if (TextureDecoder->getProperty(PropertyIndex, PropertyValue.GetData(), PropertyType, PropertySize))
							{
								FString TextureName = PropertyName;
								FString  TexturePackageName = UPackageTools::SanitizePackageName(*(ParentPackage->GetName() / TEXT("Textures") / TextureName));
								UPackage* TexturePackage = CreatePackage(*TexturePackageName);


								UTexture2D* Texture = TextureFactory->CreateTexture2D(TexturePackage, *TextureName, RF_Standalone | RF_Public);

								if (Texture)
								{
									Texture->AssetImportData->Update(Filename);
									FAssetRegistryModule::AssetCreated(Texture);

									TArray<uint8> Pixels;
									Pixels.SetNumUninitialized(Count);
									for (int PixelIndex = 0; PixelIndex < Count; PixelIndex++)
									{
										Pixels[PixelIndex] = PropertyValue[PixelIndex];
									}

									Texture->Source.Init(
										Count,
										1,
										/*NumSlices=*/ 1,
										/*NumMips=*/ 1,
										TSF_G8,
										(const uint8*)Pixels.GetData()
									);

									Texture->MipGenSettings = TMGS_NoMipmaps;
									Texture->CompressionSettings = TC_Alpha;
									Texture->SRGB = false;
									Texture->Filter = TF_Nearest;

									Texture->UpdateResource();
									Texture->PostEditChange();
									Texture->MarkPackageDirty();

									Material.SetFlakesThetaFISliceLUT(Texture);
								}
							}
						}
					}


					Log.Info(FString::Printf(TEXT("    property '%s' of type %d"), *PropertyName, PropertyType));
				}

				int TextureCount = TextureDecoder->getNumTextures();
				for (int TextureIndex = 0; TextureIndex < TextureCount; TextureIndex++)
				{
					char NameBuffer[axf::decoding::AXF_MAX_KEY_SIZE];
					TextureDecoder->getTextureName(TextureIndex, NameBuffer, axf::decoding::AXF_MAX_KEY_SIZE);
					FString TextureName(ANSI_TO_TCHAR(NameBuffer));

					Log.Info(TEXT("Importing texture: ") + TextureName);

					int Width;
					int Height;
					int Depth;
					int ChannelCount;
					int StorageTextureType;
					TextureDecoder->getTextureSize(TextureIndex, 0, Width, Height, Depth, ChannelCount, StorageTextureType);

					Log.Info(FString::Printf(TEXT("    texture '%s', (%d, %d, %d, %d)"), *TextureName, Width, Height, Depth, ChannelCount));

					TArray<float> TextureData;
					TextureData.SetNumUninitialized(Width * Height * Depth * ChannelCount);

					TextureDecoder->getTextureData(TextureIndex, 0, axf::decoding::TEXTURE_TYPE_FLOAT, TextureData.GetData());

					bool bFiltered = true;

					bool bIsColor = false;
					bool bMipmaps = true;
					bool bIsNormal = false;
					bool bIsDisplacement = false;
					bool bAtlasFor2DArray = false;
					bool bFlipGreenChannel = false;

					bool bCompressToFixedPoint = false;
					bool bHDRCompress = true;

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_DIFFUSE_COLOR
						|| TextureName == AXF_SVBRDF_TEXTURE_NAME_SPECULAR_COLOR)
					{
						bCompressToFixedPoint = true;
					}
					else if (TextureName == AXF_CARPAINT2_TEXTURE_NAME_BRDF_COLORS)
					{
						bCompressToFixedPoint = true;
						FixDiagonalBRDFColors(Width, Height, Depth, ChannelCount, TextureData);
					}

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_NORMAL
						|| TextureName == AXF_SVBRDF_TEXTURE_NAME_CLEARCOAT_NORMAL 
						// Commented due to static analysis warning (AXF_SVBRDF_TEXTURE_NAME_CLEARCOAT_NORMAL is equal to AXF_CARPAINT2_TEXTURE_NAME_CLEARCOAT_NORMAL)
						// || TextureName == AXF_CARPAINT2_TEXTURE_NAME_CLEARCOAT_NORMAL 
						)
					{
						bIsNormal = bFlipGreenChannel = true;
						bIsColor = false;
					}

					if (TextureName == AXF_CARPAINT2_TEXTURE_NAME_BTF_FLAKES)
					{
						bHDRCompress = true;
						bAtlasFor2DArray = true;
						bFiltered = false;
					}

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_ALPHA)
					{
						bIsColor = false;
					}

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_HEIGHT)
					{
						bIsColor = false;
						bIsDisplacement = true;
					}

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_SPECULAR_LOBE)
					{
					}

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_ANISO_ROTATION)
					{
					}

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_FRESNEL)
					{
						// 1x1 on samples
					}

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_CLEARCOAT_IOR)
					{
						// 1x1 or constant on samples
					}

					bool bReplaceTextureByConstant = (Width == 1) && (Height == 1);


					// specific  texture requirements:
					// - Colors(diffuse, specular, BRDF) - TC_Default(sRGB), mipmap, filtered
					// - Specular Lobes - linear(?), 1-channel(or 2-channel in case CT shading model added), 8-bit(or hdr?)
					// - Aniso rotation - dropped..., but should combine with lobes?
					// - Fresnel(always 1x1 so far - need opt), HDR else?
					// - Normal(and Clearcoat's) - TC_Normalmap, mipmaps, filtered
					// - Flakes BTF - TC_Default(sRGB) & Scale scalar, mipmaps, nearest
					// - Flakes LUT - 1D, Alpha, nomipmap, nearest
					// - Alpha - Alpha, 
					// - Height - HDR?
					// - Flakes already pow2, add padding

					FProcessedTextureSource ProcessedTextureSource;
					FFlakesTextureAtlas TextureAtlas;
					if (!bReplaceTextureByConstant)
					{
						FString  TexturePackageName = UPackageTools::SanitizePackageName(*(ParentPackage->GetName() / TEXT("Textures") / TextureName));
						UPackage* TexturePackage = CreatePackage(*TexturePackageName);
						UTexture2D* Texture = nullptr;

						Texture = TextureFactory->CreateTexture2D(TexturePackage, *TextureName, RF_Standalone | RF_Public);

						if (Texture)
						{
							Texture->AssetImportData->Update(Filename);
							FAssetRegistryModule::AssetCreated(Texture);

							FAxFTextureSource TextureSource;
							if (bAtlasFor2DArray)
							{
								CreateAtlas(Width, Height, Depth, ChannelCount, TextureData, TextureSource, TextureAtlas);
							}
							else
							{
								CreateTextureSource(TextureSource, Width, Height, ChannelCount, TextureData, bFlipGreenChannel);

								// in case we need mipmaps(that is almost always) - replace texture source with rescaled to pow2
								if (bMipmaps && !(FMath::IsPowerOfTwo(Width) && FMath::IsPowerOfTwo(Height)))
								{
									FAxFTextureSource TextureSourcePow2;
									GetTextureSourceScaledtoPow2(Texture, TextureSource, TextureSourcePow2, false);
									TextureSource = TextureSourcePow2;
								}
							}

							if (bIsNormal)
							{

								int32 PixelCount = TextureSource.GetPixelCount();
								for (int PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
								{
									FVector N = TextureSource.GetRGB(PixelIndex);
									N.Normalize();
									TextureSource.SetRGB(PixelIndex, N * 0.5f + 0.5f);
								}
							}

							if (bCompressToFixedPoint)
							{
								SetTextureSourceCompressedToFixedPoint(TextureSource, ProcessedTextureSource, Texture);
								Texture->CompressionSettings = bIsNormal ? TC_Normalmap : TC_Default;
							}
							else
							{
								SetTextureSource(Texture, TextureSource);

								Texture->CompressionSettings = bIsNormal ? TC_Normalmap : (bHDRCompress ? TC_HDR_Compressed : TC_HDR);
							}

							Texture->MipGenSettings = bMipmaps ? TMGS_FromTextureGroup : TMGS_NoMipmaps;
							Texture->SRGB = false;
							Texture->Filter = bFiltered ? TF_Default : TF_Nearest;

							Texture->UpdateResource();
							Texture->MarkPackageDirty();

							ProcessedTextureSource.SamplerType = bIsNormal ? SAMPLERTYPE_Normal : SAMPLERTYPE_LinearColor;
							ProcessedTextureSource.Texture = Texture;
						}
						else
						{
							bReplaceTextureByConstant = true;
						}
					}

					if (bReplaceTextureByConstant)
					{
						ProcessedTextureSource.Texture = nullptr;

						for (int ChannelIndex = 0; ChannelIndex < ChannelCount ; ChannelIndex++)
						{
							ProcessedTextureSource.Constant[ChannelIndex] = TextureData[ChannelIndex];
						}
					}

					Log.Info(TEXT("Done importing texture: ") + TextureName);

					float TexturePhysicalWidth, TexturePhysicalHeight;
					
					bool bIsTextureParametrized = TextureDecoder->getTextureSizeMM(TextureIndex, TexturePhysicalWidth, TexturePhysicalHeight);

					FVector2D TextureTiling(1.f, 1.f);
					if (bIsTextureParametrized)
					{
						FVector2D TextureSizeMeters = FVector2D(TexturePhysicalWidth, TexturePhysicalHeight) * 0.001f;
						TextureTiling = FVector2D(1.f, 1.f) / TextureSizeMeters * MetersPerUVUnit;
					}
					ProcessedTextureSource.Tiling = TextureTiling;

					if (TextureName == AXF_SVBRDF_TEXTURE_NAME_DIFFUSE_COLOR)
					{
						Material.SetTextureDiffuseColor(ProcessedTextureSource);
					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_ALPHA)
					{
						Material.SetTextureAlpha(ProcessedTextureSource);
					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_NORMAL)
					{
						Material.SetTextureNormal(ProcessedTextureSource);
					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_SPECULAR_COLOR)
 					{
 						Material.SetTextureSpecularColor(ProcessedTextureSource);
 					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_SPECULAR_LOBE)
					{
						Material.SetTextureSpecularLobe(ProcessedTextureSource);
					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_FRESNEL)
					{
						Material.SetTextureFresnel(ProcessedTextureSource);
					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_CLEARCOAT_NORMAL)
					{
						Material.SetTextureClearcoatNormal(ProcessedTextureSource);
					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_CLEARCOAT_IOR)
					{
						Material.SetTextureClearcoatIOR(ProcessedTextureSource);
					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_HEIGHT)
					{
						Material.SetTextureHeight(ProcessedTextureSource);
					}
					else if (TextureName == AXF_CARPAINT2_TEXTURE_NAME_BRDF_COLORS)
					{
						Material.SetTextureCarpaint2BRDFColors(ProcessedTextureSource);
					}
					else if (TextureName == AXF_CARPAINT2_TEXTURE_NAME_BTF_FLAKES)
					{
						Material.SetTextureCarpaint2BTFFlakes(ProcessedTextureSource, TextureAtlas);
					}
// Commented due to static analysis warning (AXF_SVBRDF_TEXTURE_NAME_CLEARCOAT_NORMAL is equal to AXF_CARPAINT2_TEXTURE_NAME_CLEARCOAT_NORMAL)
// 					else if (TextureName == AXF_CARPAINT2_TEXTURE_NAME_CLEARCOAT_NORMAL)
// 					{
// 						Material.SetTextureCarpaintClearcoatNormal(ProcessedTextureSource);
// 					}
					else if (TextureName == AXF_SVBRDF_TEXTURE_NAME_ANISO_ROTATION)
					{
						Log.Warn(TEXT("Anisotropic AxF materials are not supported yet - will use isotropic approximation."));
					}
					else
					{
						Log.Warn(TEXT("Texture \"") + TextureName + TEXT("\" wasn't handled"));
					}
				}

				axf::decoding::TextureDecoder::destroy(&TextureDecoder);

				Material.Build();
			}

			NewMaterial->AssetImportData = NewObject<UAssetImportData>(NewMaterial, TEXT("AssetImportData"));
			NewMaterial->AssetImportData->Update(Filename);
			UMaterialEditingLibrary::LayoutMaterialExpressions(NewMaterial);

			NewMaterial->MarkPackageDirty();
			NewMaterial->PostEditChange();

		}

		return true;
	}

	void CreateTextureSource(FAxFTextureSource& TextureSource, int Width, int Height, int ChannelCount, TArray<float> TextureData, bool bFlipGreenChannel)
	{
		TextureSource.Init(Width, Height, 1, ChannelCount);
		for (int32 PixelIndex = 0; PixelIndex < (Width * Height); PixelIndex++)
		{
			float Color[4];
			int ChannelIndex = 0;
			for (; ChannelIndex < ChannelCount; ChannelIndex++)
			{
				float SourceChannelValue = TextureData[PixelIndex * ChannelCount + ChannelIndex];
				bool bShouldFlip = bFlipGreenChannel && (ChannelIndex == 1);
				//Pixels[PixelIndex * 4 + ChannelIndex].Set(bShouldFlip ? -SourceChannelValue : SourceChannelValue);
				Color[ChannelIndex] = bShouldFlip ? -SourceChannelValue : SourceChannelValue;
			}
			//TODO: optimize texture format. For now - set other color channels to 0 and alpha to 1
			for (; ChannelIndex < 3; ChannelIndex++)
			{
				Color[ChannelIndex] = 0.0f;
			}
			if (ChannelIndex == 3)
			{
				Color[ChannelIndex] = 1.0f;
			}

			TextureSource.SetPixel(PixelIndex, FLinearColor(Color[0], Color[1], Color[2], Color[3]));
		}
	}

	void SetTextureSourceCompressedToFixedPoint(FAxFTextureSource& TextureSource, FProcessedTextureSource& ProcessedTextureSource, UTexture2D* Texture)
	{
		// Compression considerations for Axf:
		// (interesting)ranges of source pixel values:
		//   Flakes range from -3(negative?!) up to 30
		//   (diffuse)Color range -2.5(also negative?!) to 4+, SpecularColor (-6.71293, 28.4519), BRDFcolors(0, 2.9707)
		//   Height can be negative too(which is reasonable), range seen -2.25 to 2
		//   Fresnel(0.0159741, 1)
		//   AnisoRotation(-1.65286, 1.67445)
		//   ClearcoatIOR(1.24618, 1.5)
		// Should be take literal MAX or might ignore few high values for better precision for dark values? 
		// Offset + Scale?
		// In general, compressing float into is hard(impossible) unless we use some properties of existing data...

		int32 PixelCount = TextureSource.Width * TextureSource.Height;
		FVector PixelValueMin(FLT_MAX, FLT_MAX, FLT_MAX);
		FVector PixelValueMax(0.f, 0.f, 0.f);
		for (int PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
		{
			FVector PixelValue = TextureSource.GetRGB(PixelIndex);
			PixelValueMax = PixelValueMax.ComponentMax(PixelValue);
			PixelValueMin = PixelValueMin.ComponentMin(PixelValue);
		}

		// @todo Oodle : float image to RGBA16 ; just use FImage / CopyImage
		TArray<uint16> PixelsCompressed;
		if (!PixelValueMax.IsNearlyZero())
		{
			FVector Scale = PixelValueMax.ComponentMax(FVector(0.1f)); // make sure we won't normalize by too small values for some of the color components

			if ((PixelValueMin.GetMax() > 1.0f))
			{
				ProcessedTextureSource.MaxColorFactorToAvoidClamping = 1.0f / PixelValueMin.GetMax();
			}

			PixelsCompressed.SetNumUninitialized(TextureSource.Width * TextureSource.Height * 4);

			for (int PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
			{
				FVector Color = TextureSource.GetRGB(PixelIndex);

				uint16 One = 65535;

				// @todo Oodle : precompute 1/Scale then clamp in [0,1]
				FVector C = (ClampVector(Color, FVector::ZeroVector, Scale) / Scale) * One; // clamp, to avoid integer overflow(which results in nasty pixels)

				// @todo Oodle : incorrect conversion; use QuantizeUNormFloatTo16
				PixelsCompressed[PixelIndex * 4 + 0] = FMath::FloorToInt(C.X);
				PixelsCompressed[PixelIndex * 4 + 1] = FMath::FloorToInt(C.Y);
				PixelsCompressed[PixelIndex * 4 + 2] = FMath::FloorToInt(C.Z);
				PixelsCompressed[PixelIndex * 4 + 3] = One;
			}
			ProcessedTextureSource.Scale = Scale;
		}
		else
		{
			PixelsCompressed.Init(0, TextureSource.Width * TextureSource.Height * 4);
		}

		Texture->Source.Init(
			TextureSource.Width,
			TextureSource.Height,
			/*NumSlices=*/ 1,
			/*NumMips=*/ 1,
			TSF_RGBA16,
			(const  uint8*)PixelsCompressed.GetData()
		);
	}

	void CreateAtlas(int Width, int Height, int Depth, int ChannelCount, TArray<float> TextureData, FAxFTextureSource& TextureSource, FFlakesTextureAtlas& FlakesTextureAtlas)
	{
		// padding?
		int32 TargetSliceWidth = Width;
		int32 TargetSliceHeight = Height;

		int32 TargetWidth = 1;
		int32 TargetHeight = 1;
		while (true)
		{
			int32 SlicesFitIntoTexture = (TargetWidth / TargetSliceWidth) * (TargetHeight / TargetSliceHeight);

			if (SlicesFitIntoTexture >= Depth)
			{
				break;
			}
			if (TargetWidth > TargetHeight)
			{
				TargetHeight *= 2;
			}
			else
			{
				TargetWidth *= 2;
			}
		}

		int32 SlicesPerRow = (TargetWidth / TargetSliceWidth);
		int32 SlicesPerHeight = (TargetHeight / TargetSliceHeight);

		TextureSource.Init(TargetWidth, TargetHeight, 1, ChannelCount);

		for (int32 SliceIndex = 0; SliceIndex < Depth; ++SliceIndex)
		{
			int32 PixelsPerTargetSliceRow = TargetSliceHeight * TargetWidth;
			int32 SliceV = SliceIndex / SlicesPerRow;
			int32 SliceU = SliceIndex - SliceV * SlicesPerRow;
			int32 TargetSliceCornerPixelIndex = PixelsPerTargetSliceRow * SliceV + TargetSliceWidth * SliceU;

			for (int32 Y = 0; Y < Height; ++Y)
			{
				int32 TargetPixelIndex = TargetSliceCornerPixelIndex + Y * TargetWidth;
				int32 PixelIndex = Width * Height * SliceIndex + Width * Y;

				// copy row
				for (int32 X = 0; X < Width; ++X, ++PixelIndex, ++TargetPixelIndex)
				{
					float Color[4];
					int ChannelIndex = 0;
					for (; ChannelIndex < ChannelCount; ChannelIndex++)
					{
						float SourceChannelValue = TextureData[PixelIndex * ChannelCount + ChannelIndex];
						Color[ChannelIndex] = SourceChannelValue;
					}
					for (; ChannelIndex < 3; ChannelIndex++)
					{
						Color[ChannelIndex] = 0.f;
					}
					if (ChannelIndex == 3)
					{
						Color[ChannelIndex] = 1.f;
					}
					TextureSource.SetPixel(TargetPixelIndex, FLinearColor(Color[0], Color[1], Color[2], Color[3]));
				}
			}
		}

		FlakesTextureAtlas.TextureWidth = TargetWidth;
		FlakesTextureAtlas.TextureHeight = TargetHeight;
		FlakesTextureAtlas.SliceWidth = TargetSliceWidth;
		FlakesTextureAtlas.SliceHeight = TargetSliceHeight;
		FlakesTextureAtlas.FlakesTextureSlicesPerRow = (TargetWidth / TargetSliceWidth);
	}

	void SetTextureSource(UTexture2D* Texture, FAxFTextureSource TextureSource)
	{
		//@todo Oodle : just make TSF_RGBA32F , no conversion
		Texture->Source.Init(
			TextureSource.Width,
			TextureSource.Height,
			/*NumSlices=*/ 1,
			/*NumMips=*/ 1,
			TSF_RGBA16F,
			TextureSource.GetRawPixelData()
		);
	}

	void GetTextureSourceScaledtoPow2(UTexture2D* Texture, const FAxFTextureSource& TextureSource, FAxFTextureSource& TextureSourceNew, bool bUpscaleNonPow2)
	{
		FImage Image;
		Image.Init(TextureSource.Width, TextureSource.Height, ERawImageFormat::RGBA32F);
		// @todo Oodle : if TextureSource is FImage this goes away
		for (int PixelIndex = 0; PixelIndex < TextureSource.Width * TextureSource.Height; PixelIndex++)
		{
			Image.AsRGBA32F()[PixelIndex] = TextureSource.GetPixel(PixelIndex);
		}

		int32 TargetWidth;
		int32 TargetHeight;

		if (bUpscaleNonPow2)
		{
			TargetWidth = FMath::RoundUpToPowerOfTwo(TextureSource.Width);
			TargetHeight = FMath::RoundUpToPowerOfTwo(TextureSource.Height);
		}
		else
		{
			TargetWidth = 1 << FMath::FloorLog2(TextureSource.Width);
			TargetHeight = 1 << FMath::FloorLog2(TextureSource.Height);
		}

		FImage TargetImage;
		Image.ResizeTo(TargetImage, TargetWidth, TargetHeight, ERawImageFormat::RGBA32F, EGammaSpace::Linear);

		TextureSourceNew.Init(TargetImage.SizeX, TargetImage.SizeY, 1, 4);
		int32 PixelCount = TextureSourceNew.GetPixelCount();
		// @todo Oodle : if TextureSource is FImage this goes away
		for (int PixelIndex = 0; PixelIndex < PixelCount ; PixelIndex++)
		{
			TextureSourceNew.SetPixel(PixelIndex, TargetImage.AsRGBA32F()[PixelIndex]);
		}
	}

	void FixDiagonalBRDFColors(int Width, int Height, int Depth, int ChannelCount, TArray<float>& TextureData)
	{
		// fix diagonals

		for (int Y = 0; Y < Height; Y++)
		{

			int32 ZeroesStartX = Width;
			for (int32 X = Width - 1; X >= 0; X--)
			{
				int32 PixelIndex = Y * Width * Depth + X;
				bool HasNonZeroes = false;
				for (int C = 0; C < ChannelCount; C++)
				{
					// comparing floating point to zero - there are zeroes in case these diagonal BRDFColors
					HasNonZeroes = HasNonZeroes || (0.0f != TextureData[PixelIndex * ChannelCount + C]);
				}
				if (HasNonZeroes)
				{
					break;
				}
				ZeroesStartX = X;
			}

			if (ZeroesStartX < Width)
			{
				float Color[3] = { 0.0f, 0.0f, 0.0f };

				if (ZeroesStartX == 0)
				{
					if (Y != 0)
					{
						int32 PixelIndex = (Y - 1) * Width * Depth;
						for (int C = 0; C < ChannelCount; C++)
						{
							Color[C] = TextureData[PixelIndex * ChannelCount + C];
						}
					}
				}
				else
				{
					int32 PixelIndex = Y * Width * Depth + (ZeroesStartX - 1);
					for (int C = 0; C < ChannelCount; C++)
					{
						Color[C] = TextureData[PixelIndex * ChannelCount + C];
					}
				}

				for (int X = ZeroesStartX; X < Width; X++)
				{
					int32 PixelIndex = Y * Width * Depth + X;
					for (int C = 0; C < ChannelCount; C++)
					{
						TextureData[PixelIndex * ChannelCount + C] = Color[C];
					}
				}
			}
		}
	}

	void SetTextureFactory(UTextureFactory* InTextureFactory)
	{
		TextureFactory = InTextureFactory;
	}

	TMap<FString, UMaterialInterface*> GetCreatedMaterials() override
	{
		TMap<FString, UMaterialInterface*> Result;

		for (auto& Material : CreatedMaterials)
		{
			Result.Add(Material->Name, Material->GetMaterialInterface());
		}

		return Result;
	}

	const TArray<AxFImporterLogging::FLogMessage> GetLogMessages() override
	{
		return Log.Messages;
	}

	bool Reimport(const FString& InFileName, const UAxFImporterOptions& InImporterOptions, UMaterialInterface* OutMaterial)
	{
		MetersPerUVUnit = InImporterOptions.MetersPerSceneUnit;

		if (!OpenFile(InFileName, InImporterOptions))
			return false;

		UMaterial* Material = Cast<UMaterial>(OutMaterial);
		UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();
		{
			MaterialEditorOnly->BaseColor.Expression = nullptr;
			MaterialEditorOnly->EmissiveColor.Expression = nullptr;
			MaterialEditorOnly->SubsurfaceColor.Expression = nullptr;
			MaterialEditorOnly->Roughness.Expression = nullptr;
			MaterialEditorOnly->Metallic.Expression = nullptr;
			MaterialEditorOnly->Specular.Expression = nullptr;
			MaterialEditorOnly->Opacity.Expression = nullptr;
			MaterialEditorOnly->Refraction.Expression = nullptr;
			MaterialEditorOnly->OpacityMask.Expression = nullptr;
			MaterialEditorOnly->ClearCoat.Expression = nullptr;
			MaterialEditorOnly->ClearCoatRoughness.Expression = nullptr;
			MaterialEditorOnly->Normal.Expression = nullptr;
			Material->GetExpressionCollection().Empty();
		}

		for (auto& CreatedMaterialPtr : CreatedMaterials)
		{
			ICreatedMaterial& CreatedMaterial = *CreatedMaterialPtr.Get();
			FString MaterialName = ObjectTools::SanitizeObjectName(CreatedMaterial.Name);
			if (MaterialName == OutMaterial->GetName())
			{
				FString RootPackageName = *FPaths::GetPath(OutMaterial->GetOuter()->GetName());
				UPackage* RootPackage = CreatePackage(*RootPackageName);
				
				ImportMaterial(CreatedMaterial, Material, RootPackage);
				Log.Info(TEXT("Done re-importing material: ") + MaterialName);
				return true;
			}
		}
		return false;
	}

	static void LoggingCallback(int SeverityLevel, int LogContext, const wchar_t* LogMessage)
	{
		static const FString LogContexts[4] = { TEXT("AxF IO"), TEXT("AxF Decoders"), TEXT("AxF SDK"), TEXT("") };

		FString LogContextStr = LogContexts[FMath::Min(LogContext, 3)];
		FString Message = FString::Printf(TEXT("%s: %s"), *LogContextStr, LogMessage);
		switch (SeverityLevel) {
		case 0:
		{
			Log.Info(Message);
		}
		break;
		case 1:
		{
			Log.Warn(Message);
		}
		break;
		case 2:
		default:
		{
			Log.Error(Message);
		}
		}
	}


private:

	// loaded data
	FString Filename;
	axf::decoding::AXF_FILE_HANDLE FileHandle;
	int32 MaterialCount;
	TArray<TSharedPtr<ICreatedMaterial> > CreatedMaterials;
	
	// options
	float MetersPerUVUnit;

	// utils
	UTextureFactory* TextureFactory;
	static FAxFLog Log; // needs to be static for AxF SDK callback that's just a simple static function which doesn't have a way to pass context pointer
};

FAxFLog FAxFFileImporter::Log; 

FAxFImporter::FAxFImporter(const FString& PluginPath)
{
	// initialize AxF library
	const FString ThirdPartyPath = FPaths::Combine(PluginPath, TEXT("/Binaries/ThirdParty/"));
#if PLATFORM_WINDOWS
	const FString Platform = TEXT("Win64");
#elif PLATFORM_MAC
	const FString Platform = TEXT("MacOSX");
#elif PLATFORM_LINUX
	const FString Platform = TEXT("Linux");
#else
#error "Unsupported platform!"
#endif

	FString LibraryPath = FPaths::Combine(ThirdPartyPath, TEXT("AxF"), Platform);

	FPlatformProcess::PushDllDirectory(*LibraryPath);
	FString DllName = TEXT("AxFDecoding.") + FString(AFX_SDK_VERSION) + TEXT(".dll");
	AxFDecodingHandle = FPlatformProcess::GetDllHandle(*DllName);
	FPlatformProcess::PopDllDirectory(*LibraryPath);

	axf::decoding::axfEnableLogging(FAxFFileImporter::LoggingCallback, axf::decoding::LOGLEVEL_INFO);
}

FAxFImporter::~FAxFImporter()
{
	FPlatformProcess::FreeDllHandle(AxFDecodingHandle);
}

IAxFFileImporter* FAxFImporter::Create()
{
	return new FAxFFileImporter;
}

bool FAxFImporter::IsLoaded()
{
	return AxFDecodingHandle != nullptr;
}

#else // USE_AXFSDK

FAxFImporter::FAxFImporter(const FString& PluginPath)
{
}

FAxFImporter::~FAxFImporter()
{
}

IAxFFileImporter* FAxFImporter::Create()
{
	return nullptr;
}

bool FAxFImporter::IsLoaded()
{
	return false;
}

#endif // USE_AXFSDK


