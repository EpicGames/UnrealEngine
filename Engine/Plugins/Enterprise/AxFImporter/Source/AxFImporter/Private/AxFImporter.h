// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/Logging.h"

#include "AxFFileImporter.h"

class FAxFImporter
{
public:
	FAxFImporter(const FString& PluginPath);
	~FAxFImporter();

	IAxFFileImporter* Create();

	bool IsLoaded();

	void* AxFDecodingHandle = nullptr;
};
