// Copyright Epic Games, Inc. All Rights Reserved.

#include "AxFImporterFactory.h"

#include "AxFImporterModule.h"
#include "AxFImporterOptions.h"
#include "UI/AxFOptionsWindow.h"
#include "AxFImporter.h"
#include "AxFFileImporter.h"

#include "common/Logging.h"

#include "Editor.h"
#include "EditorFramework/AssetImportData.h"
#include "EngineAnalytics.h"
#include "Factories/TextureFactory.h"
#include "Framework/Application/SlateApplication.h"
#include "IMessageLogListing.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Interfaces/IMainFrameModule.h"
#include "Logging/LogMacros.h"
#include "Logging/TokenizedMessage.h"
#include "Materials/Material.h"
#include "MessageLogModule.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "UObject/StrongObjectPtr.h"

#define LOCTEXT_NAMESPACE "AxF Importer"

namespace AxFImporterImpl
{
	const TArray<FString> ListOfExtensions     = {TEXT("axf")};
	const TArray<FString> ListOfExtensionsInfo = {TEXT("AXF material files")};

	int GetMaterialCount(const FString& Filepath)
	{
		return 1;
	}

	bool ShowOptionsWindow(const FString& Filepath, const FString& PackagePath, UAxFImporterOptions& ImporterOptions)
	{
		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow                = MainFrame.GetParentWindow();
		}

		TSharedRef<SWindow> Window =
		    SNew(SWindow).Title(LOCTEXT("AxFImportOptionsTitle", "AxF Import Options")).SizingRule(ESizingRule::Autosized);

		TSharedPtr<SAxFOptionsWindow> OptionsWindow;
		Window->SetContent(
		    SAssignNew(OptionsWindow, SAxFOptionsWindow)
		        .ImportOptions(&ImporterOptions)
		        .WidgetWindow(Window)
		        .FileNameText(FText::Format(LOCTEXT("AxFImportOptionsFileName", "  Import File  :    {0}"),
		                                    FText::FromString(FPaths::GetCleanFilename(Filepath))))
		        .FilePathText(FText::FromString(Filepath))
		        .PackagePathText(FText::Format(LOCTEXT("AxFImportOptionsPackagePath", "  Import To   :    {0}"), FText::FromString(PackagePath)))
		        .MaterialCount(GetMaterialCount(Filepath)));
		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
		return OptionsWindow->ShouldImport();
	}

	void ShowLogMessages(const TArray<AxFImporterLogging::FLogMessage>& Errors)
	{
		if (Errors.Num() > 0)
		{
			FMessageLogModule&             MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			TSharedRef<IMessageLogListing> LogListing       = (MessageLogModule.GetLogListing("LoadErrors"));
			LogListing->ClearMessages();
			for (const AxFImporterLogging::FLogMessage& Error : Errors)
			{
				EMessageSeverity::Type Severity =
				    Error.Get<0>() == AxFImporterLogging::EMessageSeverity::Error ? EMessageSeverity::Error : EMessageSeverity::Warning;
				LogListing->AddMessage(FTokenizedMessage::Create(Severity, FText::FromString(Error.Get<1>())));
			}
			MessageLogModule.OpenMessageLog("LoadErrors");
		}
	}
}

UAxFImporterFactory::UAxFImporterFactory(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
    , AxFImporterModule(&IAxFImporterModule::Get())
{
	using namespace AxFImporterImpl;

#ifdef USE_AXFSDK
	bCreateNew = bText = false;
	bEditorImport      = true;
	SupportedClass     = UMaterial::StaticClass();

	ImportPriority += 100;

	Formats.Reset();
	for (int Index = 0; Index < ListOfExtensions.Num(); ++Index)
	{
		FString Extension = ListOfExtensions[Index] + TEXT(";") + ListOfExtensionsInfo[Index];
		Formats.Add(Extension);
	}
#endif
}

bool UAxFImporterFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename.ToLower());
	return AxFImporterModule->IsLoaded() && AxFImporterImpl::ListOfExtensions.Contains(Extension);
}

UObject* UAxFImporterFactory::FactoryCreateFile(UClass*           InClass,
                                                UObject*          InParent,
                                                FName             InName,
                                                EObjectFlags      InFlags,
                                                const FString&    InFilename,
                                                const TCHAR*      InParms,
                                                FFeedbackContext* Warn,
                                                bool&             bOutOperationCanceled)
{
	check(IAxFImporterModule::IsAvailable());
	
	AdditionalImportedObjects.Empty();
	if (!AxFImporterModule->GetAxFImporter().IsLoaded())
	{
		UE_LOGF(LogAxFImporter, Error, "AxF Decoding library wan't loaded!");
		return nullptr;
	}

	UObject* Object = nullptr;
#ifdef USE_AXFSDK
	TStrongObjectPtr<UAxFImporterOptions> ImporterOptions(NewObject<UAxFImporterOptions>(GetTransientPackage(), TEXT("AxF Importer Options")));

	const FString Filepath    = FPaths::ConvertRelativePathToFull(InFilename);
	const FString PackagePath = InParent->GetPathName();

	
	if (!(IsRunningCommandlet() || GIsRunningUnattendedScript || FApp::IsUnattended()))
	{
		if (!AxFImporterImpl::ShowOptionsWindow(Filepath, PackagePath, *ImporterOptions))
		{
			bOutOperationCanceled = true;
			return nullptr;
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, TEXT("AxF"));

	const uint64 StartTime = FPlatformTime::Cycles64();
    TUniquePtr<IAxFFileImporter> Importer(AxFImporterModule->GetAxFImporter().Create());

	const bool   bImported = LoadMaterials(Importer.Get(), InParent, InFlags, Filepath, *ImporterOptions, *Warn);
	if (bImported)
	{
		const TMap<FString, UMaterialInterface*> CreatedMaterials = Importer->GetCreatedMaterials();

		if (CreatedMaterials.Num() == 1)
		{
			TArray<FString> Names;
			CreatedMaterials.GetKeys(Names);
			Object = *CreatedMaterials.Find(Names[0]);
		}
		else if (CreatedMaterials.Num() != 0)
		{
			TArray<FString> Names;
			CreatedMaterials.GetKeys(Names);
			Object = (*CreatedMaterials.Find(Names[0]))->GetOutermost();
			
			for (const FString& Name : Names)
			{
				AdditionalImportedObjects.Add(*CreatedMaterials.Find(Name));
			}
		}
	}
	Importer.Reset();

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Object);

	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
	SendAnalytics(FMath::RoundToInt(ElapsedSeconds), bImported, InFilename);

	// Log time spent to import incoming file in minutes and seconds
	const int ElapsedMin = int(ElapsedSeconds / 60.f);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;

	UE_LOGF(LogAxFImporter, Log, "Imported %ls in [%d min %.3f s]", *Filepath, ElapsedMin, ElapsedSeconds);
#else
	TArray<AxFImporterLogging::FLogMessage> Error;
	Error.Emplace(AxFImporterLogging::EMessageSeverity::Error, TEXT("AxF Plugin was not compiled with the AxF SDK!"));
	AxFImporterImpl::ShowLogMessages(Error);
	bOutOperationCanceled = true;
#endif

	return Object;
}

void UAxFImporterFactory::SendAnalytics(int32 ImportDurationInSeconds, bool bImportSuccess, const FString& InFilename) const
{
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		FString ImporterName = TEXT("AxF");

		EventAttributes.Emplace(TEXT("ImporterType"), ImporterName);
		EventAttributes.Emplace(TEXT("ImportDuration"), ImportDurationInSeconds);

		if (bImportSuccess)
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("AxF.Import"), EventAttributes);
		}
		else
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("AxF.ImportFail"), EventAttributes);
		}
	}
}

void UAxFImporterFactory::CleanUp()
{
	Super::CleanUp();
	FReimportManager::Instance()->UnregisterHandler(*this);
}

bool UAxFImporterFactory::LoadMaterials(IAxFFileImporter* Importer,
    UObject* InPackage, EObjectFlags InFlags, const FString& InFilename, const UAxFImporterOptions& InImporterOptions, FFeedbackContext& OutWarn)
{
	FScopedSlowTask Progress(100.0f, LOCTEXT("ImportingFile", "Importing File ..."), true, OutWarn);
	Progress.MakeDialog();

	const float StartProgress = 25.f;
	Progress.EnterProgressFrame(StartProgress, FText::Format(LOCTEXT("LoadingMaterials", "Loading materials: {0} ..."), FText::FromString(InFilename)));

	bool bSuccess = Importer->OpenFile(InFilename, InImporterOptions);

	if (bSuccess)
	{
		// TODO: show materials selection

		const int MaterialCount = Importer->GetMaterialCountInFile();
		auto      ProgressFunc  = [StartProgress, &Progress, &InFilename, MaterialCount](const FString& MaterialName, int MaterialIndex)  //
		{
			if (MaterialIndex >= 0)
			{
				const float FinishProgress   = 90.f;
				const float MaterialProgress = (FinishProgress - StartProgress) / MaterialCount;
				Progress.EnterProgressFrame(MaterialProgress,
				                            FText::Format(LOCTEXT("MaterialProcessing", "Material processing: {0} ..."), FText::FromString(MaterialName)));
			}
			else
			{
				Progress.EnterProgressFrame(5.f, FText::Format(LOCTEXT("FinishingMaterials", "Finishing materials: {0} ..."), FText::FromString(InFilename)));
			}
		};

		TStrongObjectPtr<UTextureFactory> TextureFactory(NewObject<UTextureFactory>());
		// processes and any AxF materials
		Importer->SetTextureFactory(TextureFactory.Get());
		bSuccess &= Importer->ImportMaterials(InPackage, InFlags, ProgressFunc);
		if (!bSuccess)
		{
			OutWarn.Logf(ELogVerbosity::Error, TEXT("Failed to import materials for: '%s'"), *InFilename);
		}
	}
	else
	{
		OutWarn.Logf(ELogVerbosity::Error, TEXT("Failed to load: '%s': failed to find or load the module"), *InFilename);
	}

	const TArray<AxFImporterLogging::FLogMessage> Messages = Importer->GetLogMessages();
	AxFImporterImpl::ShowLogMessages(Messages);

	return bSuccess;
}

bool UAxFImporterFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UMaterialInterface* Material = Cast<UMaterialInterface>(Obj);
	if (Material != nullptr)
	{
		if (Material->AssetImportData)
		{
			Material->AssetImportData->ExtractFilenames(OutFilenames);

			const FString FileName(Material->AssetImportData->GetFirstFilename());
			const FString Extension = FPaths::GetExtension(FileName);
			if (AxFImporterImpl::ListOfExtensions.FindByPredicate([&Extension](const FString& Current) { return Current == Extension; }) != nullptr)
				return true;
		}
	}

	return false;
}

EReimportResult::Type UAxFImporterFactory::Reimport(UObject* Obj)
{
	if (!Obj || !Obj->IsA(UMaterialInterface::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	if (!AxFImporterModule->GetAxFImporter().IsLoaded())
	{
		UE_LOGF(LogAxFImporter, Error, "AxF Decoding library wan't loaded!");
		return EReimportResult::Failed;
	}

	TArray<AxFImporterLogging::FLogMessage> Errors;
#ifdef USE_AXFSDK
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetReimport(Obj);

	UMaterialInterface* Material    = Cast<UMaterialInterface>(Obj);

    TUniquePtr<IAxFFileImporter> Importer(AxFImporterModule->GetAxFImporter().Create());

	const FString FileName(Material->AssetImportData->GetFirstFilename());
	const FString Extension = FPaths::GetExtension(FileName);
	check(AxFImporterImpl::ListOfExtensions.FindByPredicate([&Extension](const FString& Current) { return Current == Extension; }) != nullptr);
	if (!FileName.Len())
	{
		// If there is no file path provided, can't reimport from source.
		Errors.Emplace(AxFImporterLogging::EMessageSeverity::Error,
		               FString::Printf(TEXT("Attempt to reimport material with empty file name: %s"), *Material->GetName()));
	}
	if (!FPaths::FileExists(FileName))
	{
		Errors.Emplace(AxFImporterLogging::EMessageSeverity::Error,
		               FString::Printf(TEXT("Attempt to reimport material with non existing file name: %s"), *FileName));
	}
	if (Errors.Num() == 0)
	{
		TStrongObjectPtr<UAxFImporterOptions> ImporterOptions(NewObject<UAxFImporterOptions>(GetTransientPackage(), TEXT("AxF Importer Options")));
		TStrongObjectPtr<UTextureFactory>     TextureFactory(NewObject<UTextureFactory>());
		Importer->SetTextureFactory(TextureFactory.Get());
		if (!Importer->Reimport(FileName, *ImporterOptions, Material))
		{
			Errors.Emplace(AxFImporterLogging::EMessageSeverity::Error, FString::Printf(TEXT("Failed to reimport material: %s"), *Material->GetName()));
		}
	}

	TArray<AxFImporterLogging::FLogMessage> Messages = Importer->GetLogMessages();
	Messages.Append(Errors);
	AxFImporterImpl::ShowLogMessages(Messages);

#else
	Errors.Emplace(AxFImporterLogging::EMessageSeverity::Error, TEXT("AxF Plugin was not compiled with the AxF SDK!"));
	AxFImporterImpl::ShowLogMessages(Errors);
#endif

	return Errors.Num() == 0 ? EReimportResult::Succeeded : EReimportResult::Failed;
}

void UAxFImporterFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if (NewReimportPaths.Num() == 0)
		return;

	UMaterialInterface* Material = Cast<UMaterialInterface>(Obj);
	if (Material != nullptr)
	{
		if (FactoryCanImport(NewReimportPaths[0]))
		{
			Material->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
		}
	}
}

#undef LOCTEXT_NAMESPACE
