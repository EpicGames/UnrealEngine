// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorReimportHandler.h"
#include "Factories/Factory.h"

#include "AxFImporterFactory.generated.h"

class IAxFImporterModule;
class UAxFImporterOptions;
class IAxFFileImporter;
class UTextureFactory;
class FFeedbackContext;

UCLASS(HideCategories = Object, CollapseCategories)
class UAxFImporterFactory : public UFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()
public:
	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;

	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename,
	                                   const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;

	virtual void CleanUp() override;
	//~ End UFactory Interface

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;

	virtual EReimportResult::Type Reimport(UObject* Obj) override;

	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	//~ End FReimportHandler Interface

private:
	bool LoadMaterials(IAxFFileImporter* Importer, UObject* InPackage, EObjectFlags InFlags, const FString& InFilename, const UAxFImporterOptions& InImporterOptions,
	                   FFeedbackContext& Warn);

	void SendAnalytics(int32 ImportDurationInSeconds, bool bImportSuccess, const FString& InFilename) const;

private:
	IAxFImporterModule* AxFImporterModule;
};
