// Copyright Epic Games, Inc. All Rights Reserved.

#include "AxFImporterModule.h"

#include "AxFImporter.h"
#include "AxFFileImporter.h"

#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "EditorFramework/AssetImportData.h"
#include "EditorReimportHandler.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Misc/Paths.h"
#include "Interfaces/IPluginManager.h"

#define LOCTEXT_NAMESPACE "AxF Importer"

const TCHAR* IAxFImporterModule::ModuleName = TEXT("AxFImporter");

class FAxFImporterModule : public IAxFImporterModule
{
public:
	virtual FAxFImporter& GetAxFImporter() override;
	virtual IAxFFileImporter* CreateFileImporter() override;

	bool IsLoaded() override
	{
		return GetAxFImporter().IsLoaded();
	}


private:
	void AddContentBrowserContextMenuExtender();
	void RemoveContentBrowserContextMenuExtender();

	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets);
	static void                  ExecuteReimportMaterials(TArray<FAssetData> AssetData);

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	static const FString PluginPath;

	TUniquePtr<FAxFImporter> AxFImporter;
	FDelegateHandle          ContentBrowserExtenderDelegateHandle;
};

const FString FAxFImporterModule::PluginPath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Enterprise/AxFImporter")); //TODO: check

FAxFImporter& FAxFImporterModule::GetAxFImporter()
{
	// lazy init the importer
	if (!AxFImporter)
		AxFImporter.Reset(new FAxFImporter(PluginPath));
	return *AxFImporter;
}

IAxFFileImporter* FAxFImporterModule::CreateFileImporter()
{
	return GetAxFImporter().Create();
}


void FAxFImporterModule::StartupModule()
{
	if (!IsRunningCommandlet())
	{
		AddContentBrowserContextMenuExtender();
	}
}

void FAxFImporterModule::ShutdownModule()
{
	if (!IsRunningCommandlet())
	{
		RemoveContentBrowserContextMenuExtender();
	}

	AxFImporter.Reset();
}

void FAxFImporterModule::AddContentBrowserContextMenuExtender()
{
#ifdef USE_AXFSDK
	if (!IsRunningCommandlet())
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();

		CBMenuExtenderDelegates.Add(
		    FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&FAxFImporterModule::OnExtendContentBrowserAssetSelectionMenu));
		ContentBrowserExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
	}
#endif
}

void FAxFImporterModule::RemoveContentBrowserContextMenuExtender()
{
	if (ContentBrowserExtenderDelegateHandle.IsValid() && FModuleManager::Get().IsModuleLoaded("ContentBrowser"))
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
		CBMenuExtenderDelegates.RemoveAll([this](const FContentBrowserMenuExtender_SelectedAssets& Delegate) {
			return Delegate.GetHandle() == ContentBrowserExtenderDelegateHandle;
		});
	}
}

TSharedRef<FExtender> FAxFImporterModule::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	// Run through the assets to determine if any meet our criteria
	bool bShouldExtendAssetActions = false;
	for (const FAssetData& Asset : SelectedAssets)
	{
		if (Asset.AssetClassPath == UMaterial::StaticClass()->GetClassPathName() || Asset.AssetClassPath == UMaterialInstance::StaticClass()->GetClassPathName())
		{
			// Need to load the material at this point to figure out the type of the AssetImportData
			UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Asset.GetAsset());

			if (MaterialInterface && MaterialInterface->AssetImportData)
			{
				const FString FileName(MaterialInterface->AssetImportData->GetFirstFilename());
				const FString Extension = FPaths::GetExtension(FileName);
				if (Extension == TEXT("axf"))
				{
					bShouldExtendAssetActions = true;
					break;
				}
			}
		}
	}

	if (bShouldExtendAssetActions)
	{
		// Add the AxF material actions sub-menu extender
		Extender->AddMenuExtension(
		    "GetAssetActions",
		    EExtensionHook::After,
		    nullptr,
		    FMenuExtensionDelegate::CreateLambda(
		        [SelectedAssets](FMenuBuilder& MenuBuilder)  //
		        {
			        // Add the reimport AxF material sub-menu extender
			        MenuBuilder.AddMenuEntry(
			            NSLOCTEXT("AssetTypeActions_Material", "ObjectContext_ReimportAxFMaterial", "Reimport Material"),
			            NSLOCTEXT("AssetTypeActions_Material", "ObjectContext_ReimportAxFMaterialTooltip", "Reimport the selected material(s)."),
			            FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset"),
			            FUIAction(FExecuteAction::CreateStatic(&FAxFImporterModule::ExecuteReimportMaterials, SelectedAssets), FCanExecuteAction()));
		        }));
	}

	return Extender;
}

void FAxFImporterModule::ExecuteReimportMaterials(TArray<FAssetData> SelectedAssets)
{
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.AssetClassPath == UMaterial::StaticClass()->GetClassPathName() || AssetData.AssetClassPath == UMaterialInstance::StaticClass()->GetClassPathName())
		{
			UObject* AssetToReimport = AssetData.GetAsset();

			FReimportManager::Instance()->Reimport(AssetToReimport, /*bAskForNewFileIfMissing=*/false);
		}
	}
}

IMPLEMENT_MODULE(FAxFImporterModule, AxFImporter);

#undef LOCTEXT_NAMESPACE
