// Copyright Epic Games, Inc. All Rights Reserved.

#include "AxFImporterOptions.h"

UAxFImporterOptions::UAxFImporterOptions(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
{
	MetersPerSceneUnit = 0.16f;
}

