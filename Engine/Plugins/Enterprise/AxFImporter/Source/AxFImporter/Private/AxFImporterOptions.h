// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Engine/EngineTypes.h"
#include "UObject/ObjectMacros.h"

#include "AxFImporterOptions.generated.h"

UCLASS(config = Engine, defaultconfig)
class AXFIMPORTER_API UAxFImporterOptions : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	// hidden properties
	UPROPERTY(config, meta = (DisplayName = "Meters per scene unit", ToolTip = "The conversion ratio between meters and scene units for materials.",
	                          ClampMin = 0.01, ClampMax = 1000))
	float MetersPerSceneUnit;
};
