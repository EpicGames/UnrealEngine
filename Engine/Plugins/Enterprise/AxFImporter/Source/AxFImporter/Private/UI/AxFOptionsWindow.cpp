// Copyright Epic Games, Inc. All Rights Reserved.

#include "AxFOptionsWindow.h"

#include "Styling/AppStyle.h"
#include "HAL/PlatformProcess.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "AxFOptionsWindow"

void SAxFOptionsWindow::Construct(const FArguments& InArgs)
{
	ImportOptions = InArgs._ImportOptions;
	Window = InArgs._WidgetWindow;
	bShouldImport = false;

	TSharedPtr<SBox> DetailsViewBox;
	TSharedPtr<SInlineEditableTextBlock> FileNameLabel;
	const FText VersionText(FText::Format(LOCTEXT("AxFOptionWindow_Version", " Version   {0}"), FText::FromString("1.0")));
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f, 10.0f)
		.AutoHeight()
		[
			SAssignNew(FileNameLabel, SInlineEditableTextBlock)
			.IsReadOnly(true)
			.Text(InArgs._FileNameText)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SInlineEditableTextBlock)
			.IsReadOnly(true)
			.Text(InArgs._PackagePathText)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(DetailsViewBox, SBox)
			.MinDesiredHeight(80.0f)
			.MinDesiredWidth(240.0f)
		]
		+ SVerticalBox::Slot()
		.MaxHeight(50)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(5)
			+ SUniformGridPanel::Slot(0, 0)
			.HAlign(HAlign_Left)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(5)
				+ SUniformGridPanel::Slot(0, 0)
				.HAlign(HAlign_Left)
				[
					SNew(SInlineEditableTextBlock)
					.IsReadOnly(true)
					.Text(VersionText)
				]
				+ SUniformGridPanel::Slot(1, 0)
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.WidthOverride(16)
					.HeightOverride(16)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush("Icons.Help"))
						.OnMouseButtonDown(this, &SAxFOptionsWindow::OnHelp)
					]
				]
			]
			+ SUniformGridPanel::Slot(1, 0)
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(5)
				+ SUniformGridPanel::Slot(0, 0)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("AxFOptionWindow_ImportMaterials", "Import"))
					.ToolTipText(LOCTEXT("AxFOptionWindow_ImportMaterials_ToolTip", "Import the file and add to the current Level"))
					.OnClicked(this, &SAxFOptionsWindow::OnImport)
				]
				+ SUniformGridPanel::Slot(1, 0)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("AxFOptionWindow_Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("AxFOptionWindow_Cancel_ToolTip", "Cancel importing this file"))
					.OnClicked(this, &SAxFOptionsWindow::OnCancel)
				]
			]
		]
	];

	// Add file's full path and material count as a tooltip
	{
		FString Label = InArgs._FilePathText.ToString();
		if (InArgs._MaterialCount == 1)
		{
			Label += TEXT("\none material");
		}
		else
		{
			Label += TEXT("\n") + FString::FromInt(InArgs._MaterialCount) + TEXT(" materials");
		}
		FileNameLabel->SetToolTipText(FText::FromString(Label));
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsViewBox->SetContent(DetailsView.ToSharedRef());
	DetailsView->SetObject(ImportOptions);
}

FReply SAxFOptionsWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) /*override*/
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}

FReply SAxFOptionsWindow::OnImport()
{
	bShouldImport = true;
	if (Window.IsValid())
	{
		Window.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}


FReply SAxFOptionsWindow::OnCancel()
{
	bShouldImport = false;
	if (Window.IsValid())
	{
		Window.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SAxFOptionsWindow::OnHelp(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
