// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/SlateApplication.h"
#include "Widgets/SCompoundWidget.h"

class SAxFOptionsWindow : public SCompoundWidget
{
public:
    SLATE_BEGIN_ARGS(SAxFOptionsWindow) {}

    SLATE_ARGUMENT(UObject*, ImportOptions);
    SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow);
    SLATE_ARGUMENT(FText, FileNameText);
    SLATE_ARGUMENT(FText, FilePathText);
    SLATE_ARGUMENT(FText, PackagePathText);
    SLATE_ARGUMENT(int, MaterialCount);
    SLATE_END_ARGS();

public:
    void Construct(const FArguments& InArgs);

    virtual bool SupportsKeyboardFocus() const override
    {
        return true;
    }
    virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

    bool ShouldImport() const;

private:
    FReply OnImport();
    FReply OnCancel();
    FReply OnHelp(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent);

private:
    UObject*          ImportOptions;
    TWeakPtr<SWindow> Window;
    bool              bShouldImport;
};

inline bool SAxFOptionsWindow::ShouldImport() const
{
    return bShouldImport;
}
