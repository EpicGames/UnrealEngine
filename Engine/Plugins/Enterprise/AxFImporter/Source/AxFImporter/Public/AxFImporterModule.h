// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"

class FAxFImporter;
class IAxFFileImporter;

class AXFIMPORTER_API IAxFImporterModule : public IModuleInterface
{
public:
	static const TCHAR* ModuleName;

	virtual FAxFImporter& GetAxFImporter() = 0;

	virtual IAxFFileImporter* CreateFileImporter() = 0;

	virtual bool IsLoaded() = 0;

	static IAxFImporterModule& Get();

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable();

};

inline bool IAxFImporterModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(ModuleName);
}

inline IAxFImporterModule& IAxFImporterModule::Get()
{
	return FModuleManager::LoadModuleChecked<IAxFImporterModule>(ModuleName);
}
